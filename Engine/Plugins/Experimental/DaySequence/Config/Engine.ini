[CoreRedirects]
+PackageRedirects=(OldName="/Script/TimeOfDay", NewName="/Script/DaySequence")
+ClassRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequence", NewName="/Script/DaySequence.DaySequence")
+ClassRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequencePlayer", NewName="/Script/DaySequence.DaySequencePlayer")
+ClassRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequenceDirector", NewName="/Script/DaySequence.DaySequenceDirector")
+StructRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequenceBindingReference", NewName="/Script/DaySequence.DaySequenceBindingReference")
+StructRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequenceBindingReferenceArray", NewName="/Script/DaySequence.DaySequenceBindingReferenceArray")
+StructRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequenceBindingReferences", NewName="/Script/DaySequence.DaySequenceBindingReferences")
+ClassRedirects=(OldName="/Script/TimeOfDay.TimeOfDaySequenceTrack", NewName="/Script/DaySequence.DaySequenceTrack")
+ClassRedirects=(OldName="/Script/TimeOfDay.TimeOfDayActor", NewName="/Script/DaySequence.DaySequenceActor")
+ClassRedirects=(OldName="/Script/TimeOfDay.EnvironmentLightingActor", NewName="/Script/DaySequence.EnvironmentLightingActor")
+EnumRedirects=(OldName="/Script/DaySequence.EDaySequenceModifierBlendMode", NewName="/Script/DaySequence.EDaySequenceModifierMode", ValueChanges=(("None","Global"),("Distance","Volume")))
