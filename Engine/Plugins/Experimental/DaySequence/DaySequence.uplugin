{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DaySequence",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DaySequenceEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DaySequence",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		}
	]
}
