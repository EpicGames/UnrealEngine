// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/BaseDaySequenceActor.h"

#include "DaySequenceSubsystem.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "Engine/StaticMesh.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseDaySequenceActor)

ABaseDaySequenceActor::ABaseDaySequenceActor(const FObjectInitializer& Init)
: Super(Init)
{
	SunRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SunRoot"));
	SunRootComponent->SetupAttachment(RootComponent);
	
	SunComponent = CreateDefaultSubobject<UDirectionalLightComponent>(TEXT("Sun"));
	SunComponent->SetupAttachment(SunRootComponent);
	
	ExponentialHeightFogComponent = CreateOptionalDefaultSubobject<UExponentialHeightFogComponent>(TEXT("ExponentialHeightFog"));
	if(ExponentialHeightFogComponent)
	{
		ExponentialHeightFogComponent->SetupAttachment(RootComponent);
		ExponentialHeightFogComponent->bEnableVolumetricFog = true;
	}

	SkyAtmosphereComponent = CreateDefaultSubobject<USkyAtmosphereComponent>(TEXT("SkyAtmosphere"));
	SkyAtmosphereComponent->SetupAttachment(RootComponent);

	SkyLightComponent = CreateDefaultSubobject<USkyLightComponent>(TEXT("SkyLight"));
	SkyLightComponent->SetupAttachment(RootComponent);
	SkyLightComponent->bRealTimeCapture = true;
	SkyLightComponent->bLowerHemisphereIsBlack = false;

	VolumetricCloudComponent = CreateOptionalDefaultSubobject<UVolumetricCloudComponent>(TEXT("VolumetricCloud"));
	if(VolumetricCloudComponent)
	{
    	VolumetricCloudComponent->SetupAttachment(RootComponent);
	}

	static ConstructorHelpers::FObjectFinder<UStaticMesh> SkySphereDefaultMesh(TEXT("/Engine/EngineSky/SM_SkySphere.SM_SkySphere"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> SkySphereDefaultMaterial(TEXT("/Engine/EngineSky/M_SimpleSkyDome.M_SimpleSkyDome"));
	SkySphereComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("SkySphere"));
	if(SkySphereComponent)
	{
		SkySphereComponent->SetupAttachment(RootComponent);
		SkySphereComponent->SetStaticMesh(SkySphereDefaultMesh.Object);
		SkySphereComponent->SetMaterial(0, SkySphereDefaultMaterial.Object.Get());
		SkySphereComponent->SetRelativeScale3D(FVector(400.f));
	}
}

void ABaseDaySequenceActor::BeginPlay()
{
	Super::BeginPlay();
	
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			DaySequenceSubsystem->SetDaySequenceActor(this);
		}
	}
}

void ABaseDaySequenceActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			if (DaySequenceSubsystem->GetDaySequenceActor(/* bFindFallbackOnNull */ false) != this)
			{
				DaySequenceSubsystem->SetDaySequenceActor(this);
			}
		}
	}
}
