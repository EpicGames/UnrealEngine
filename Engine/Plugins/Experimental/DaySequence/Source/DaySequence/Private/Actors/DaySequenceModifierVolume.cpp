// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/DaySequenceModifierVolume.h"

#include "DaySequenceActor.h"
#include "DaySequenceModifierComponent.h"
#include "DaySequenceSubsystem.h"

#include "Components/BoxComponent.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceModifierVolume)

namespace UE::DaySequence
{
	static TAutoConsoleVariable<bool> CVarModifierVolumeEnableSplitscreenSupport(
	TEXT("DaySequence.ModifierVolume.EnableSplitscreenSupport"),
	true,
	TEXT("When true, Day Sequence Modifier Volumes attempt to initialize transient modifier components for all local players."),
	ECVF_Default
	);

	static TAutoConsoleVariable<bool> CVarModifierVolumeCanDisableModifiersDuringSplitscreen(
	TEXT("DaySequence.ModifierVolume.CanDisableModifiersDuringSplitscreen"),
	true,
	TEXT("When true and splitscreen support is not enabled, the modifier volume will disable all managed modifier components."),
	ECVF_Default
	);

	static TAutoConsoleVariable<bool> CVarModifierVolumeCanReplaceBlendTargets(
	TEXT("DaySequence.ModifierVolume.CanReplaceBlendTargets"),
	true,
	TEXT("When true, the modifier will attempt to detect when a local player is assigned a new player controller and will use the new controller for the associated modifier's blend target."),
	ECVF_Default
	);
}

ADaySequenceModifierVolume::ADaySequenceModifierVolume(const FObjectInitializer& Init)
: Super(Init)
, bEnableSplitscreenSupport(false)
{
	PrimaryActorTick.bCanEverTick = false;

#if WITH_EDITORONLY_DATA
	// This is not generally safe. It is only safe if the modifier is in volume mode and
	// the spatial loading threshold distance is greater than the radius of the volume.
	bIsSpatiallyLoaded = false;
#endif
	
	DaySequenceModifier = CreateDefaultSubobject<UDaySequenceModifierComponent>(TEXT("DaySequenceModifier"));
	SetRootComponent(DaySequenceModifier);

	DefaultBox = CreateDefaultSubobject<UBoxComponent>(TEXT("Box"));
	DefaultBox->SetupAttachment(DaySequenceModifier);
	DefaultBox->SetLineThickness(10.f);
	DefaultBox->SetBoxExtent(FVector(500.f));
	DefaultBox->SetGenerateOverlapEvents(false);
	DefaultBox->SetCollisionProfileName(TEXT("NoCollision"));
	DefaultBox->CanCharacterStepUpOn = ECB_No;

	FComponentReference DefaultBoxReference;
	DefaultBoxReference.ComponentProperty = TEXT("DefaultBox");
	DaySequenceModifier->AddVolumeShapeComponent(DefaultBoxReference);
}

void ADaySequenceModifierVolume::BeginPlay()	
{
	Super::BeginPlay();

	Initialize();
}

void ADaySequenceModifierVolume::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Deinitialize();
	
	Super::EndPlay(EndPlayReason);
}

void ADaySequenceModifierVolume::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	
	Initialize();
}

void ADaySequenceModifierVolume::Initialize()
{
	if (IsTemplate())
	{
		return;
	}
	
	// This actor should only initialize on the client.
	if (GetNetMode() == NM_DedicatedServer)
	{
		SetActorEnableCollision(false);
		return;
	}

	if (DaySequenceModifier)
	{
		DaySequenceModifier->GetOnPostEnableModifier().AddUniqueDynamic(this, &ADaySequenceModifierVolume::OnPostEnableModifier);
	}

	if (const UWorld* World = GetWorld())
	{
#if WITH_EDITOR
		if (World->WorldType == EWorldType::Editor)
		{
			DaySequenceActor = nullptr;
			if (IsValid(DaySequenceModifier))
			{
				DaySequenceModifier->UnbindFromDaySequenceActor();
			}
		}
#endif

		DaySequenceActorSetup();
		
		if (World->IsGameWorld())
		{
			auto HandleNewPlayerController = [](ADaySequenceModifierVolume* DSMV, APlayerController* PlayerController)
			{
				if (DSMV && PlayerController->IsLocalController())
				{
					DSMV->CreatePlayer(PlayerController);
				}
			};

			// Create players that exist initially. We may not find anything here, but the actor spawned handler below should handle anything we miss.
			for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				if (APlayerController* PlayerController = Iterator->Get())
				{
					HandleNewPlayerController(this, PlayerController);
				}
			}

			ActorSpawnedHandle = World->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateWeakLambda(this, [this, HandleNewPlayerController, World](AActor* SpawnedActor)
			{
				if (APlayerController* PlayerController = Cast<APlayerController>(SpawnedActor))
				{
					World->GetTimerManager().SetTimerForNextTick([WeakThis = TWeakObjectPtr(this), HandleNewPlayerController, PlayerController]()
					{
						if (TStrongObjectPtr<ADaySequenceModifierVolume> PinnedThis = WeakThis.Pin())
						{
							HandleNewPlayerController(PinnedThis.Get(), PlayerController);
						}
					});
				}
			}));
		}

		if (World->IsPlayingReplay())
		{
			ReplayScrubbedHandle = FNetworkReplayDelegates::OnReplayScrubComplete.AddWeakLambda(this, [this](const UWorld* InWorld)
			{
				if (InWorld == GetWorld())
				{
					DaySequenceActorSetup();
				}
			});
		}
	}
}

void ADaySequenceModifierVolume::Deinitialize()
{
	if (const UWorld* World = GetWorld())
	{
		if (ActorSpawnedHandle.IsValid())
		{
			World->RemoveOnActorSpawnedHandler(ActorSpawnedHandle);
			ActorSpawnedHandle.Reset();
		}
		
		if (ReplayScrubbedHandle.IsValid())
		{
			FNetworkReplayDelegates::OnReplayScrubComplete.Remove(ReplayScrubbedHandle);
			ReplayScrubbedHandle.Reset();
		}
	}

	if (DaySequenceModifier)
	{
		DaySequenceModifier->GetOnPostEnableModifier().RemoveAll(this);
	}
}

void ADaySequenceModifierVolume::DaySequenceActorSetup()
{
	SetupDaySequenceSubsystemCallbacks();
	BindToDaySequenceActor();
}

void ADaySequenceModifierVolume::BindToDaySequenceActor()
{
	auto RebindAllModifiers = [this]()
	{
		DaySequenceModifier->BindToDaySequenceActor(DaySequenceActor);
		for (auto AdditionalPlayerIterator = AdditionalPlayers.CreateIterator(); AdditionalPlayerIterator; ++AdditionalPlayerIterator)
		{
			if(UDaySequenceModifierComponent* ModifierComponent = AdditionalPlayerIterator->Value)
			{
				ModifierComponent->BindToDaySequenceActor(DaySequenceActor);
			}
		}

		OnDaySequenceActorBound(DaySequenceActor);
	};
	
	if (const UWorld* World = GetWorld())
	{
		if (const UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			if (ADaySequenceActor* NewActor = DaySequenceSubsystem->GetDaySequenceActor())
			{
				if (NewActor != DaySequenceActor)
				{
					DaySequenceActor = NewActor;
					RebindAllModifiers();
				}
			}
		}
	}
}

void ADaySequenceModifierVolume::SetupDaySequenceSubsystemCallbacks()
{
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			// Prevent consecutive calls to this function from adding redundant lambdas to invocation list.
			if (!DaySequenceSubsystem->OnDaySequenceActorSetEvent.IsBoundToObject(this))
			{
				DaySequenceSubsystem->OnDaySequenceActorSetEvent.AddWeakLambda(this, [this](ADaySequenceActor* InActor)
				{
					BindToDaySequenceActor();
				});
			}
		}
	}
}

void ADaySequenceModifierVolume::CreatePlayer(APlayerController* InPC)
{
	if (!InPC || !InPC->GetLocalPlayer())
	{
		return;
	}

	if (UE::DaySequence::CVarModifierVolumeCanReplaceBlendTargets.GetValueOnAnyThread())
	{
		// See if the primary player controller is being replaced.
		if (CachedPlayer == InPC->Player)
		{
			DaySequenceModifier->SetBlendTarget(InPC);
			return;
		}

		// See if any additional player controllers are being replaced.
		for (TPair<TObjectPtr<UPlayer>, TObjectPtr<UDaySequenceModifierComponent>> AdditionalPlayer : AdditionalPlayers)
		{
			if (AdditionalPlayer.Key == InPC->Player)
			{
				if (AdditionalPlayer.Value) { AdditionalPlayer.Value->SetBlendTarget(InPC); }
				return;
			}
		}
	}

	UDaySequenceModifierComponent* PlayerModifier = nullptr;

	if (CachedPlayer == nullptr)
	{
		CachedPlayer = InPC->Player;
		PlayerModifier = DaySequenceModifier;
	}
	else if (IsSplitscreenSupported())
	{
        PlayerModifier = DuplicateObject<UDaySequenceModifierComponent>(DaySequenceModifier, this, TEXT("AdditionalPlayerModifier"));
		AdditionalPlayers.FindOrAdd(InPC->Player) = PlayerModifier;
	}
	else
	{
		// If CachedPlayer is not null and we don't support splitscreen, just track the player.
		AdditionalPlayers.FindOrAdd(InPC->Player) = nullptr;
	}

	if (PlayerModifier)
	{
		if (!PlayerModifier->IsRegistered())
		{
			// This happens for duplicated modifiers.
			PlayerModifier->RegisterComponent();
			PlayerModifier->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
			PlayerModifier->SetBias(DaySequenceModifier->GetBias() + AdditionalPlayers.Num());
		}

		PlayerModifier->SetBlendTarget(InPC);
		PlayerModifier->BindToDaySequenceActor(DaySequenceActor);
	}
	else if (!IsSplitscreenSupported() && UE::DaySequence::CVarModifierVolumeCanDisableModifiersDuringSplitscreen.GetValueOnAnyThread())
	{
		// If we get here, we have multiple local players and do not have splitscreen support enabled and our CVar state allows us to disable the managed modifiers.
		// In this case, we will just disable all modifiers.
		
		for (TPair<TObjectPtr<UPlayer>, TObjectPtr<UDaySequenceModifierComponent>> AdditionalPlayer : AdditionalPlayers)
		{
			if (AdditionalPlayer.Value)
			{
				AdditionalPlayer.Value->DisableComponent();
			}
		}
		
		if (DaySequenceModifier)
		{
			DaySequenceModifier->DisableComponent();
		}
	}
}

bool ADaySequenceModifierVolume::IsSplitscreenSupported() const
{
	return bEnableSplitscreenSupport && UE::DaySequence::CVarModifierVolumeEnableSplitscreenSupport.GetValueOnAnyThread();
}
