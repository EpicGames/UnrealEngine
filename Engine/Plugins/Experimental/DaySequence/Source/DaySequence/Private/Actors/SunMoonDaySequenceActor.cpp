// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/SunMoonDaySequenceActor.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DaySequenceCollectionAsset.h"

#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SunMoonDaySequenceActor)

ASunMoonDaySequenceActor::ASunMoonDaySequenceActor(const FObjectInitializer& Init)
: Super(Init)
{
	MoonComponent = CreateDefaultSubobject<UDirectionalLightComponent>(TEXT("Moon"));
	MoonComponent->SetupAttachment(SunRootComponent);

	// Give Sun forward shading priority.
	SunComponent->SetForwardShadingPriority(1);
	MoonComponent->SetForwardShadingPriority(0);

	// Configure other Moon defaults.
	MoonComponent->SetAtmosphereSunLightIndex(1);	// Make Moon the secondary directional light that contributes to the sky atmosphere.
	MoonComponent->SetIntensity(0.05f);
	MoonComponent->SetUseTemperature(true);
	MoonComponent->SetTemperature(9000.f);
	MoonComponent->SetWorldRotation(FRotator(-45.f, 0.f, 0.f));
	
	// Override the sky sphere material.
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> SkySphereDefaultMaterial(TEXT("/DaySequence/MI_24hrSky.MI_24hrSky"));
	SkySphereComponent->SetMaterial(0, SkySphereDefaultMaterial.Object.Get());

	// Override the default collection (which animates the moon and sky material)
	static ConstructorHelpers::FObjectFinder<UDaySequenceCollectionAsset> DefaultCollection(TEXT("/DaySequence/DSCA_24hr.DSCA_24hr"));
	DaySequenceCollections.Add(DefaultCollection.Object.Get());
}
