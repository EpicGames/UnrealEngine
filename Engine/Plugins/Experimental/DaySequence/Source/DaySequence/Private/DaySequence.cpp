// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequence.h"
#include "DaySequenceDirector.h"
#include "DaySequencePlayer.h"
#include "DaySequenceTrack.h"
#include "DaySequenceModule.h"
#include "DaySequenceCameraModifier.h"

#include "Animation/AnimInstance.h"
#include "Components/ActorComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/AssetUserData.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "MovieSceneSpawnable.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneDataLayerTrack.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Tracks/MovieSceneSlomoTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequence)

#define LOCTEXT_NAMESPACE "DaySequence"

#if WITH_EDITOR
UDaySequence::FPostDuplicateEvent UDaySequence::PostDuplicateEvent;
#endif

static TAutoConsoleVariable<int32> CVarDefaultLockEngineToDisplayRate(
	TEXT("DaySequence.DefaultLockEngineToDisplayRate"),
	0,
	TEXT("0: Playback locked to playback frames\n1: Unlocked playback with sub frame interpolation"),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultTickResolution(
	TEXT("DaySequence.DefaultTickResolution"),
	TEXT("24000fps"),
	TEXT("Specifies the default tick resolution for newly created Day sequences. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultDisplayRate(
	TEXT("DaySequence.DefaultDisplayRate"),
	TEXT("30fps"),
	TEXT("Specifies the default display frame rate for newly created Day sequences; also defines frame locked frame rate where sequences are set to be frame locked. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarDefaultClockSource(
	TEXT("DaySequence.DefaultClockSource"),
	0,
	TEXT("Specifies the default clock source for newly created Day sequences. 0: Tick, 1: Platform, 2: Audio, 3: RelativeTimecode, 4: Timecode, 5: Custom"),
	ECVF_Default);

UDaySequence::UDaySequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
{
	bParentContextsAreSignificant = true;
}

void UDaySequence::Initialize()
{
	Initialize(RF_Transactional);
}

void UDaySequence::Initialize(EObjectFlags Flags)
{
	MovieScene = NewObject<UMovieScene>(this, NAME_None, Flags);
	if (!(Flags & RF_Transactional))
	{
		// UMovieScene::PostInitProperties always sets RF_Transactional.
		// For transient procedural sequences, this is not desirable. Explicitly
		// clear RF_Transactional for these cases.
		MovieScene->ClearFlags(RF_Transactional);
	}
	
	const bool bFrameLocked = CVarDefaultLockEngineToDisplayRate.GetValueOnGameThread() != 0;
	MovieScene->SetEvaluationType( bFrameLocked ? EMovieSceneEvaluationType::FrameLocked : EMovieSceneEvaluationType::WithSubFrames );

	FFrameRate TickResolution(60000, 1);
	TryParseString(TickResolution, *CVarDefaultTickResolution.GetValueOnGameThread());
	MovieScene->SetTickResolutionDirectly(TickResolution);

	FFrameRate DisplayRate(30, 1);
	TryParseString(DisplayRate, *CVarDefaultDisplayRate.GetValueOnGameThread());
	MovieScene->SetDisplayRate(DisplayRate);

	int32 ClockSource = CVarDefaultClockSource.GetValueOnGameThread();
	MovieScene->SetClockSource((EUpdateClockSource)ClockSource);
}

void UDaySequence::AddDefaultBinding(const FGuid& PossessableGuid)
{
	BindingReferences.AddDefaultBinding(PossessableGuid);
}

void UDaySequence::AddSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization)
{
	if (Specialization == EDaySequenceBindingReferenceSpecialization::None)
	{
		return;
	}

	auto GetBindingName = [Specialization]() -> FString
	{
		static const FString RootBindingName = LOCTEXT("RootBindingName", "Root Day Sequence Actor").ToString();
		static const FString CameraModifierBindingName = LOCTEXT("CameraModifierBindingName", "Day Sequence Camera Modifier").ToString();
		
		switch (Specialization)
		{
		case EDaySequenceBindingReferenceSpecialization::Root:
			return RootBindingName;
			
		case EDaySequenceBindingReferenceSpecialization::CameraModifier:
			return CameraModifierBindingName;
			
		case EDaySequenceBindingReferenceSpecialization::None:
		default:
			ensureMsgf(true, TEXT("Invalid specialization provided, assuming Root specialization!"));
			return RootBindingName;
		}
	};
	
	auto GetClass = [Specialization]() -> UClass*
	{
		switch (Specialization)
		{
		case EDaySequenceBindingReferenceSpecialization::Root:
			return ADaySequenceActor::StaticClass();
			
		case EDaySequenceBindingReferenceSpecialization::CameraModifier:
			return UDaySequenceCameraModifier::StaticClass();
			
		case EDaySequenceBindingReferenceSpecialization::None:
		default:
			ensureMsgf(true, TEXT("Invalid specialization provided, assuming Root specialization!"));
			return ADaySequenceActor::StaticClass();
		}
	};
	
	// only 1 allowed
	if (BindingReferences.FindSpecializedBinding(Specialization).IsValid())
	{
		return;
	}
	
	// Add a default binding
	FGuid PossessableGuid = MovieScene->AddPossessable(GetBindingName(), GetClass());
	BindingReferences.AddSpecializedBinding(PossessableGuid, Specialization);
}

FGuid UDaySequence::GetSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization) const
{
	return BindingReferences.FindSpecializedBinding(Specialization);
}

UObject* UDaySequence::MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName)
{
	return MovieSceneHelpers::MakeSpawnableTemplateFromInstance(InSourceObject, MovieScene, ObjectName);
}

bool UDaySequence::CanAnimateObject(UObject& InObject) const 
{
	return InObject.IsA<AActor>() || InObject.IsA<UActorComponent>() || InObject.IsA<UAnimInstance>();
}

#if WITH_EDITOR

ETrackSupport UDaySequence::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (InTrackClass == UMovieScene3DAttachTrack::StaticClass() ||
		InTrackClass == UMovieScene3DPathTrack::StaticClass() ||
		InTrackClass == UMovieSceneAudioTrack::StaticClass() ||
		InTrackClass == UMovieSceneEventTrack::StaticClass() ||
		InTrackClass == UMovieSceneLevelVisibilityTrack::StaticClass() ||
		InTrackClass == UMovieSceneDataLayerTrack::StaticClass() ||
		InTrackClass == UMovieSceneMaterialParameterCollectionTrack::StaticClass() ||
		InTrackClass == UMovieSceneSlomoTrack::StaticClass() ||
		InTrackClass == UMovieSceneSpawnTrack::StaticClass() ||
		InTrackClass == UMovieSceneTimeWarpTrack::StaticClass() ||
		InTrackClass == UMovieSceneCVarTrack::StaticClass() ||
		InTrackClass == UDaySequenceTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}

bool UDaySequence::IsFilterSupportedImpl(const FString& InFilterName) const
{
	static const TArray<FString> SupportedFilters = {
		TEXT("Audio"),
		TEXT("Folder")
	};
	return SupportedFilters.Contains(InFilterName);
}

void UDaySequence::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (DirectorBlueprint)
	{
		DirectorBlueprint->GetAssetRegistryTags(Context);
	}
#endif

	Super::GetAssetRegistryTags(Context);
}
#endif

void UDaySequence::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITORONLY_DATA
	if (DirectorBlueprint)
	{
		DirectorClass = DirectorBlueprint->GeneratedClass.Get();

		// Remove the binding for the director blueprint recompilation and re-add it to be sure there is only one entry in the list
		DirectorBlueprint->OnCompiled().RemoveAll(this);
		DirectorBlueprint->OnCompiled().AddUObject(this, &UDaySequence::OnDirectorRecompiled);
	}
	else
	{
		DirectorClass = nullptr;
	}
#endif

#if WITH_EDITOR
	if (PostDuplicateEvent.IsBound())
	{
		PostDuplicateEvent.Execute(this);
	}
#endif
}

void UDaySequence::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	BindingReferences.PerformLegacyFixup();
#endif

#if WITH_EDITOR
	if (!DirectorBlueprint)
	{
		UBlueprint* PhantomDirector = FindObject<UBlueprint>(this, TEXT("SequenceDirector"));
		if (!ensureMsgf(!PhantomDirector, TEXT("Phantom sequence director found in sequence '%s' which has a nullptr DirectorBlueprint. Re-assigning to prevent future crash."), *GetName()))
		{
			DirectorBlueprint = PhantomDirector;
		}
	}

	if (DirectorBlueprint)
	{
		DirectorBlueprint->ClearFlags(RF_Standalone);

		// Remove the binding for the director blueprint recompilation and re-add it to be sure there is only one entry in the list
		DirectorBlueprint->OnCompiled().RemoveAll(this);
		DirectorBlueprint->OnCompiled().AddUObject(this, &UDaySequence::OnDirectorRecompiled);

		if (DirectorBlueprint->Rename(*GetDirectorBlueprintName(), nullptr, (REN_NonTransactional | REN_DoNotDirty | REN_Test)))
		{
			DirectorBlueprint->Rename(*GetDirectorBlueprintName(), nullptr, (REN_NonTransactional | REN_DoNotDirty));
		}
	}

	TSet<FGuid> InvalidSpawnables;

	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(Index);
		if (!Spawnable.GetObjectTemplate())
		{
			if (Spawnable.GeneratedClass_DEPRECATED && Spawnable.GeneratedClass_DEPRECATED->ClassGeneratedBy)
			{
				const FName TemplateName = MakeUniqueObjectName(MovieScene, UObject::StaticClass(), Spawnable.GeneratedClass_DEPRECATED->ClassGeneratedBy->GetFName());

				UObject* NewTemplate = NewObject<UObject>(MovieScene, Spawnable.GeneratedClass_DEPRECATED->GetSuperClass(), TemplateName);
				if (NewTemplate)
				{
					Spawnable.CopyObjectTemplate(*NewTemplate, *this);
				}
			}
		}

		if (!Spawnable.GetObjectTemplate())
		{
			InvalidSpawnables.Add(Spawnable.GetGuid());
			UE_LOGF(LogDaySequence, Warning, "Spawnable '%ls' with ID '%ls' does not have a valid object template", *Spawnable.GetName(), *Spawnable.GetGuid().ToString());
		}
	}
#endif
}

void UDaySequence::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (MovieScene)
	{
		// Remove any invalid object bindings
		TSet<FGuid> ValidObjectBindings;
		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			ValidObjectBindings.Add(MovieScene->GetSpawnable(Index).GetGuid());
		}
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			ValidObjectBindings.Add(MovieScene->GetPossessable(Index).GetGuid());
		}

		BindingReferences.RemoveInvalidBindings(ValidObjectBindings);
	}
#endif
}

bool UDaySequence::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	bool bRetVal = Super::Rename(NewName, NewOuter, Flags);

#if WITH_EDITOR
	if (DirectorBlueprint)
	{
		DirectorBlueprint->Rename(*GetDirectorBlueprintName(), this, Flags);
	}
#endif

	return bRetVal;
}

void UDaySequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	if (Context)
	{
		BindingReferences.AddBinding(ObjectId, &PossessedObject, Context);
	}
}

bool UDaySequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	return Object.IsA<AActor>() || Object.IsA<UActorComponent>() || Object.IsA<UAnimInstance>();
}

void UDaySequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	BindingReferences.ResolveBinding(ObjectId, Context, OutObjects);
}

FGuid UDaySequence::FindBindingFromObject(UObject* InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	if (InObject)
	{
		if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			FMovieSceneSequenceID SequenceID = EvaluationState->FindSequenceId(this);
			return EvaluationState->FindCachedObjectId(*InObject, SequenceID, SharedPlaybackState);
		}
	}
	return FGuid();
}

void UDaySequence::GatherExpiredObjects(const FMovieSceneObjectCache& InObjectCache, TArray<FGuid>& OutInvalidIDs) const
{
	for (const FGuid& ObjectId : BindingReferences.GetBoundAnimInstances())
	{
		for (TWeakObjectPtr<> WeakObject : InObjectCache.IterateBoundObjects(ObjectId))
		{
			UAnimInstance* AnimInstance = Cast<UAnimInstance>(WeakObject.Get());
			if (!AnimInstance || !AnimInstance->GetOwningComponent() || AnimInstance->GetOwningComponent()->GetAnimInstance() != AnimInstance)
			{
				OutInvalidIDs.Add(ObjectId);
			}
		}
	}
}

UMovieScene* UDaySequence::GetMovieScene() const
{
	return MovieScene;
}

UObject* UDaySequence::GetParentObject(UObject* Object) const
{
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		return Component->GetOwner();
	}

	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Object))
	{
		if (AnimInstance->GetWorld())
		{
			return AnimInstance->GetOwningComponent();
		}
	}

	return nullptr;
}

bool UDaySequence::AllowsSpawnableObjects() const
{
#if WITH_EDITOR
	if (!UMovieScene::IsTrackClassAllowed(UMovieSceneSpawnTrack::StaticClass()))
	{
		return false;
	}
#endif
	return true;
}

bool UDaySequence::CanRebindPossessable(const FMovieScenePossessable& InPossessable) const
{
	return !InPossessable.GetParent().IsValid();
}

void UDaySequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
	BindingReferences.RemoveBinding(ObjectId);
}

void UDaySequence::UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext)
{
	BindingReferences.RemoveObjects(ObjectId, InObjects, InContext);
}

void UDaySequence::UnbindInvalidObjects(const FGuid& ObjectId, UObject* InContext)
{
	BindingReferences.RemoveInvalidObjects(ObjectId, InContext);
}

#if WITH_EDITOR

UBlueprint* UDaySequence::GetDirectorBlueprint() const
{
	return DirectorBlueprint;
}

FString UDaySequence::GetDirectorBlueprintName() const
{
	return GetDisplayName().ToString() + "_DirectorBP";
}

void UDaySequence::SetDirectorBlueprint(UBlueprint* NewDirectorBlueprint)
{
	if (DirectorBlueprint)
	{
		DirectorBlueprint->OnCompiled().RemoveAll(this);
	}

	DirectorBlueprint = NewDirectorBlueprint;

	if (DirectorBlueprint)
	{
		DirectorClass = NewDirectorBlueprint->GeneratedClass.Get();
		DirectorBlueprint->OnCompiled().AddUObject(this, &UDaySequence::OnDirectorRecompiled);
	}
	else
	{
		DirectorClass = nullptr;
	}

	MarkAsChanged();
}

void UDaySequence::OnDirectorRecompiled(UBlueprint* InCompiledBlueprint)
{
	ensure(InCompiledBlueprint == DirectorBlueprint);
	DirectorClass = DirectorBlueprint->GeneratedClass.Get();

	MarkAsChanged();
}

FGuid UDaySequence::FindOrAddBinding(UObject* InObject)
{
	using namespace UE::MovieScene;

	UObject* PlaybackContext = InObject ? InObject->GetWorld() : nullptr;
	if (!InObject || !PlaybackContext)
	{
		return FGuid();
	}

	AActor* Actor = Cast<AActor>(InObject);
	// @todo: sequencer-python: need to figure out how we go from a spawned object to an object binding without the spawn register or any IMovieScenePlayer interface
	// Normally this process would happen through sequencer, since it has more context than just the sequence asset.
	// For now we cannot possess spawnables or anything within them since we have no way of retrieving the spawnable from the object
	if (Actor && Actor->ActorHasTag("SequencerActor"))
	{
		TOptional<FMovieSceneSpawnableAnnotation> Annotation = FMovieSceneSpawnableAnnotation::Find(Actor);
		if (Annotation.IsSet() && Annotation->OriginatingSequence == this)
		{
			return Annotation->ObjectBindingID;
		}

		UE_LOGF(LogDaySequence, Error, "Unable to possess object '%ls' since it is, or is part of a spawnable that is not in this sequence.", *InObject->GetName());
		return FGuid();
	}

	UObject* ParentObject = GetParentObject(InObject);
	FGuid    ParentGuid   = ParentObject ? FindOrAddBinding(ParentObject) : FGuid();

	if (ParentObject && !ParentGuid.IsValid())
	{
		UE_LOGF(LogDaySequence, Error, "Unable to possess object '%ls' because it's parent could not be bound.", *InObject->GetName());
		return FGuid();
	}

	// Perform a potentially slow lookup of every possessable binding in the sequence to see if we already have this
	{
		FSharedPlaybackStateCreateParams CreateParams;
		CreateParams.PlaybackContext = PlaybackContext;
		TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*this, CreateParams);

		FMovieSceneEvaluationState State;
		TransientPlaybackState->AddCapabilityRaw(&State);
		State.AssignSequence(MovieSceneSequenceID::Root, *this, TransientPlaybackState);

		FGuid ExistingID = State.FindObjectId(*InObject, MovieSceneSequenceID::Root, TransientPlaybackState);
		if (ExistingID.IsValid())
		{
			return ExistingID;
		}
	}

	// We have to possess this object
	if (!CanPossessObject(*InObject, PlaybackContext))
	{
		return FGuid();
	}

	FString NewName = Actor ? Actor->GetActorLabel() : InObject->GetName();

	const FGuid NewGuid = MovieScene->AddPossessable(NewName, InObject->GetClass());

	// Attempt to use the parent as a context if necessary
	UObject* BindingContext = ParentObject && AreParentContextsSignificant() ? ParentObject : PlaybackContext;

	// Set up parent/child guids for possessables within spawnables
	if (ParentGuid.IsValid())
	{
		FMovieScenePossessable* ChildPossessable = MovieScene->FindPossessable(NewGuid);
		if (ensure(ChildPossessable))
		{
			ChildPossessable->SetParent(ParentGuid, MovieScene);
		}

		FMovieSceneSpawnable* ParentSpawnable = MovieScene->FindSpawnable(ParentGuid);
		if (ParentSpawnable)
		{
			ParentSpawnable->AddChildPossessable(NewGuid);
		}
	}

	BindPossessableObject(NewGuid, *InObject, BindingContext);

	return NewGuid;

}

FGuid UDaySequence::CreatePossessable(UObject* ObjectToPossess)
{
	return FindOrAddBinding(ObjectToPossess);
}

FGuid UDaySequence::CreateSpawnable(UObject* ObjectToSpawn)
{
	if (!MovieScene || !ObjectToSpawn)
	{
		return FGuid();
	}

	TArray<TSharedRef<IMovieSceneObjectSpawner>> ObjectSpawners;

	// In order to create a spawnable, we have to instantiate all the relevant object spawners for sequences, and try to create a spawnable from each
	FDaySequenceModule& DaySequenceModule = FModuleManager::LoadModuleChecked<FDaySequenceModule>("DaySequence");
	DaySequenceModule.GenerateObjectSpawners(ObjectSpawners);

	// The first object spawner to return a valid result will win
	for (TSharedRef<IMovieSceneObjectSpawner> Spawner : ObjectSpawners)
	{
		TValueOrError<FNewSpawnable, FText> Result = Spawner->CreateNewSpawnableType(*ObjectToSpawn, *MovieScene, nullptr);
		if (Result.IsValid())
		{
			FNewSpawnable& NewSpawnable = Result.GetValue();

			NewSpawnable.Name = MovieSceneHelpers::MakeUniqueSpawnableName(MovieScene, NewSpawnable.Name);			

			FGuid NewGuid = MovieScene->AddSpawnable(NewSpawnable.Name, *NewSpawnable.ObjectTemplate);

			UMovieSceneSpawnTrack* NewSpawnTrack = MovieScene->AddTrack<UMovieSceneSpawnTrack>(NewGuid);
			if (NewSpawnTrack)
			{
				NewSpawnTrack->AddSection(*NewSpawnTrack->CreateNewSection());
			}
			return NewGuid;
		}
	}

	return FGuid();
}

#endif // WITH_EDITOR

UObject* UDaySequence::CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID)
{
	UObject* DirectorOuter = SharedPlaybackState->GetPlaybackContext();

	UDaySequencePlayer* DaySequencePlayer = nullptr;
	IMovieScenePlayer* OptionalPlayer = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
	if (ensure(OptionalPlayer))
	{
		DaySequencePlayer = Cast<UDaySequencePlayer>(OptionalPlayer->AsUObject());
	}

	if (DirectorClass && DirectorOuter && DirectorClass->IsChildOf(UDaySequenceDirector::StaticClass()))
	{
		FName DirectorName = NAME_None;

#if WITH_EDITOR
		// Give it a pretty name so it shows up in the debug instances drop down nicely
		DirectorName = MakeUniqueObjectName(DirectorOuter, DirectorClass, *(GetFName().ToString() + TEXT("_Director")));
#endif

		UDaySequenceDirector* NewDirector = NewObject<UDaySequenceDirector>(DirectorOuter, DirectorClass, DirectorName, RF_Transient);
		NewDirector->SubSequenceID = SequenceID.GetInternalValue();
		NewDirector->Player = DaySequencePlayer;
		NewDirector->MovieScenePlayerIndex = OptionalPlayer ? OptionalPlayer->GetUniqueIndex() : INDEX_NONE;
		NewDirector->OnCreated();
		return NewDirector;
	}

	return nullptr;
}

void UDaySequence::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != NULL)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UDaySequence::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void UDaySequence::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UDaySequence::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#undef LOCTEXT_NAMESPACE