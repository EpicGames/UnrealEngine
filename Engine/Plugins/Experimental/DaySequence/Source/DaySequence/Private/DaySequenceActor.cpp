// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceActor.h"
#include "DaySequenceCameraModifier.h"
#include "DaySequenceCollectionAsset.h"
#include "DaySequence.h"
#include "DaySequenceModule.h"
#include "DaySequencePlayer.h"
#include "DaySequenceTrack.h"
#include "DaySequenceConditionTag.h"
#include "DaySequenceSubsystem.h"
#include "DaySequenceStaticTime.h"

#include "Components/BillboardComponent.h"
#include "Curves/CurveFloat.h"
#include "Engine/Texture2D.h"
#include "LevelSequenceActor.h" // For FBoundActorProxy
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneBindingOverrides.h"
#include "Net/UnrealNetwork.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Package.h"
#include "UObject/UObjectThreadContext.h" // For FUObjectThreadContext
#include "ProfilingDebugging/CsvProfiler.h"

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
#include "Engine/Engine.h"	// GEngine
#include "GameFramework/HUD.h"	// AHUD::OnShowDebugInfo
#include "Engine/Canvas.h"	// UCanvas
#endif

#include "Engine/World.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceActor)

#if WITH_EDITOR
ADaySequenceActor::FOnSubSectionRemovedEvent ADaySequenceActor::OnSubSectionRemovedEvent;
#endif

namespace UE::DaySequence
{
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	FDaySequenceDebugEntry::FDaySequenceDebugEntry(FShowDebugDataConditionFunction InShowCondition, FGetDebugDataFunction InGetData, FGetNumericValue InGetNumericValue)
	: ShowCondition(InShowCondition)
	, GetData(InGetData)
	, GetNumericValue(InGetNumericValue)
	{}
	
	int32 GDaySequenceDebugLevel = 2;
	FAutoConsoleVariableRef CVarDaySequenceActorDebugLevel(
		TEXT("DaySequence.DebugLevel"),
		GDaySequenceDebugLevel,
		TEXT("The debug level to use.")
	);
#endif
	
	int32 GFrameBudgetMicroseconds = 30;
	FAutoConsoleVariableRef CVarFrameBudgetMicroseconds(
		TEXT("TimeOfDay.FrameBudget"),
		GFrameBudgetMicroseconds,
		TEXT("(Default: 30us) Approximate max per-frame budget for time-of-day actors in microseconds.")
	);

	static TAutoConsoleVariable<bool> CVarDumpStackTraceOnSetTimeOfDay(
		TEXT("DaySequence.DumpStackTraceOnSetTimeOfDay"),
		false,
		TEXT("When true, calls to SetTimeOfDay will dump the stack trace."),
		ECVF_Default
	);
} // namespace UE::DaySequence

ADaySequenceActor::ADaySequenceActor(const FObjectInitializer& Init)
: Super(Init)
, SequenceUpdateInterval(0.f)
, bRunDayCycle(true)
, bUseInterpCurve(false)
, DayLength(24, 0, 0)
, TimePerCycle(0, 5, 0)
, InitialTimeOfDay(6, 0, 0)
, StaticTimeManager(MakeShared<UE::DaySequence::FStaticTimeManager>())
{
	USceneComponent* SceneRootComponent = CreateDefaultSubobject<USceneComponent>(USceneComponent::GetDefaultSceneRootVariableName());
	SetRootComponent(SceneRootComponent);

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics() : DecalTexture(TEXT("/Engine/EditorResources/S_LevelSequence")) {}
		};
		static FConstructorStatics ConstructorStatics;

		if (UBillboardComponent* LocalSpriteComponent = GetSpriteComponent())
		{
			LocalSpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
			LocalSpriteComponent->SetupAttachment(RootComponent);
			LocalSpriteComponent->SetUsingAbsoluteScale(true);
			LocalSpriteComponent->bReceivesDecals = false;
			LocalSpriteComponent->bHiddenInGame = true;
		}
	}

	TimeOfDayPreview = FDaySequenceTime(6, 0, 0);
#endif // WITH_EDITORONLY_DATA

	// The DaySequenceActor is ticked separately in LevelTick. However, in Editor, we tick to allow
	// deferred initialization of the root sequence outside of actor construction / BP reinstancing.
	// Comment from Nick: We also now tick in dev builds + Editor to catch changes to
	// GDaySequenceDebugLevel which is set via a CVar.
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_DuringPhysics;
	PrimaryActorTick.EndTickGroup = TG_DuringPhysics;
	
	// SequencePlayer must be a default sub object for it to be replicated correctly
	SequencePlayer = Init.CreateDefaultSubobject<UDaySequencePlayer>(this, "AnimationPlayer", true);
	BindingOverrides = Init.CreateDefaultSubobject<UMovieSceneBindingOverrides>(this, "BindingOverrides");
	CameraModifierManager = Init.CreateDefaultSubobject<UDaySequenceCameraModifierManager>(this, "CameraModifierManager");

	bAlwaysRelevant = true;
	bReplicates = true;
	bReplicateUsingRegisteredSubObjectList = true;
	bReplicatePlayback = true;
	SetHidden(false);
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (!IsTemplate())
	{
		AHUD::OnShowDebugInfo.AddUObject(this, &ADaySequenceActor::OnShowDebugInfo);
	}
#endif

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &ADaySequenceActor::HandleConditionReinstanced);
#endif
}

IDaySequencePlayer* ADaySequenceActor::GetSequencePlayer() const
{
	return GetSequencePlayerInternal();
}

void ADaySequenceActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (HasAuthority())
	{
		SetReplicates(bReplicatePlayback);
	}

	if (IsValid(SequencePlayer))
	{
		SequencePlayer->OnSequenceUpdated().AddUObject(this, &ADaySequenceActor::OnSequencePlayerUpdate);
	}
	
	InitializePlayer();
}

#if WITH_EDITOR
void ADaySequenceActor::OnConstruction(const FTransform& Transform)
{
	// It is unsafe to update the root sequence (incl. its delegates)
	// during actor construction. Defer to the next tick.
	UpdateRootSequenceOnTickState.bUpdate = true;
}
#endif

void ADaySequenceActor::Tick(float DeltaTime)
{
	check(!HasAnyFlags(RF_ClassDefaultObject));
	Super::Tick(DeltaTime);
	
#if WITH_EDITOR
	if (UpdateRootSequenceOnTickState.bUpdate && GetWorld() != nullptr && GetWorld()->WorldType == EWorldType::Editor)
	{
		UpdateRootSequence(UpdateRootSequenceOnTickState.Mode);
		UpdateRootSequenceOnTickState = FUpdateRootSequenceState();
	}
#endif

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (UE::DaySequence::GDaySequenceDebugLevel != CachedDebugLevel)
	{
		OnDebugLevelChanged.Broadcast(UE::DaySequence::GDaySequenceDebugLevel);
		CachedDebugLevel = UE::DaySequence::GDaySequenceDebugLevel;
	}
#endif

	// Can only occur in game worlds (proper games and PIE)
	if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		Player->Tick(DeltaTime);
	}

#if WITH_EDITOR
	if (CameraModifierManager)
	{
		CameraModifierManager->UpdateEditorPreview();
	}
#endif
}

bool ADaySequenceActor::ShouldTickIfViewportsOnly() const
{
	return GetWorld() != nullptr && GetWorld()->WorldType == EWorldType::Editor;
}

UDaySequencePlayer* ADaySequenceActor::GetSequencePlayerInternal() const
{
	return SequencePlayer && SequencePlayer->IsValid() ? SequencePlayer : nullptr;
}

bool ADaySequenceActor::ContainsDaySequence(const UDaySequence* InDaySequence)
{
	if (InDaySequence)
	{
		for (UDaySequenceCollectionAsset* Collection : DaySequenceCollections)
		{
			if (!Collection)
			{
				continue;
			}
			
			for (const FDaySequenceCollectionEntry& SequenceAsset : Collection->DaySequences)
			{
				if (SequenceAsset.Sequence == InDaySequence)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void ADaySequenceActor::SetReplicatePlayback(bool bInReplicatePlayback)
{
	bReplicatePlayback = bInReplicatePlayback;
	SetReplicates(bReplicatePlayback);
}

void ADaySequenceActor::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ADaySequenceActor, SequencePlayer);
	DOREPLIFETIME(ADaySequenceActor, DayInterpCurve);
	DOREPLIFETIME(ADaySequenceActor, bUseInterpCurve);
}

void ADaySequenceActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Initialize our transient preview time to InitialTimeOfDay on load.
	// Only do this for editor world, in PIE world we want to preserve the value in case we are overriding initial time of day.
	if (const UWorld* World = GetWorld(); World && World->WorldType == EWorldType::Editor)
	{
		TimeOfDayPreview = InitialTimeOfDay;
	}
	
	// Build our root sequence after load to ensure that the editor can parse the root sequence
	// hierarchy for editing binding overrides. This is only necessary for editor, since the
	// root sequence will be initialized in PostInitializeComponents()/InitializePlayer() for runtime.
	InitializeRootSequence();

	SubSections.Empty();

	UpdateRootSequenceOnTickState.bUpdate = true;
#endif

#if WITH_EDITORONLY_DATA
	// Fix sprite component so that it's attached to the root component. In the past, the sprite component was the root component.
	UBillboardComponent* LocalSpriteComponent = FindComponentByClass<UBillboardComponent>();
	if (LocalSpriteComponent && LocalSpriteComponent->GetAttachParent() != RootComponent)
	{
		LocalSpriteComponent->SetupAttachment(RootComponent);
	}
#endif
}

void ADaySequenceActor::BeginDestroy()
{
	Super::BeginDestroy();
}

void ADaySequenceActor::BeginPlay()
{
	Super::BeginPlay();

	if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		AddReplicatedSubObject(Player);

		// Only play if we have a valid day sequence.
		if (HasValidRootSequence())
		{
			// Always play a valid day sequence. Pause if bRunDayCycle is false
			// to allow sequence spawnables and property tracks to be set from initial
			// time of day.
			Player->PlayLooping();
			
#if WITH_EDITORONLY_DATA
			const bool bPause = !bRunDayCycle || bOverrideRunDayCycle;
#else
			const bool bPause = !bRunDayCycle;
#endif
			
			if (bPause)
			{
				Player->Pause();
			}
		}
	}
}

void ADaySequenceActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	StopDaySequenceUpdateTimer();
	
	if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		RemoveReplicatedSubObject(Player);

		// Stop may modify a lot of actor state so it needs to be called
		// during EndPlay (when Actors + World are still valid) instead
		// of waiting for the UObject to be destroyed by GC.
		Player->Stop();

		Player->OnPlay.RemoveAll(this);
		Player->OnPlayReverse.RemoveAll(this);
		Player->OnPause.RemoveAll(this);
		Player->OnStop.RemoveAll(this);

		Player->TearDown();
	}

	Super::EndPlay(EndPlayReason);
}

void ADaySequenceActor::RewindForReplay()
{
	if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		Player->RewindForReplay();
	}
}

void ADaySequenceActor::Destroyed()
{
#if WITH_EDITOR
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			if (DaySubsystem->GetDaySequenceActor() == this)
			{
				DaySubsystem->SetDaySequenceActor(nullptr);
			}
		}
	}
#endif

	Super::Destroyed();
}

void ADaySequenceActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	if (Ar.IsLoading())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Transfer DaySequenceCollection (deprecated) to DaySequenceCollections
		if (DaySequenceCollection)
		{
			// When transferring to DaySequenceCollections, clear the array to preserve
			// child class override behavior of DaySequenceCollection.
			DaySequenceCollections.Reset();
			DaySequenceCollections.Add(DaySequenceCollection);
			DaySequenceCollection = nullptr;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}


bool ADaySequenceActor::RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (const TMap<FGuid, TFunction<bool(TArray<UObject*, TInlineAllocator<1>>&)>>* Entry = BindingResolveFunctions.Find(InSequenceID))
	{
		if (const TFunction<bool(TArray<UObject*, TInlineAllocator<1>>&)>* ResolveFunc = Entry->Find(InBindingId))
		{
			return (*ResolveFunc)(OutObjects);
		}
	}
	
	// todo: make this an ensureMsgf? This is null when duping a DSA BP.
	if (BindingOverrides)
	{
		return BindingOverrides->LocateBoundObjects(InBindingId, InSequenceID, OutObjects);
	}

	return true;
}

UObject* ADaySequenceActor::GetInstanceData() const
{
	return nullptr;
}

bool ADaySequenceActor::GetIsReplicatedPlayback() const
{
	return bReplicatePlayback;
}

#if WITH_EDITOR
TSharedPtr<FStructOnScope> ADaySequenceActor::GetObjectPickerProxy(TSharedPtr<IPropertyHandle> ObjectPropertyHandle)
{
	TSharedRef<FStructOnScope> Struct = MakeShared<FStructOnScope>(FBoundActorProxy::StaticStruct());
	reinterpret_cast<FBoundActorProxy*>(Struct->GetStructMemory())->Initialize(ObjectPropertyHandle);
	return Struct;
}

void ADaySequenceActor::UpdateObjectFromProxy(FStructOnScope& Proxy, IPropertyHandle& ObjectPropertyHandle)
{
	UObject* BoundActor = reinterpret_cast<FBoundActorProxy*>(Proxy.GetStructMemory())->BoundActor;
	ObjectPropertyHandle.SetValue(BoundActor);
}

UMovieSceneSequence* ADaySequenceActor::RetrieveOwnedSequence() const
{
	return RootSequence;
}

bool ADaySequenceActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	for (UDaySequenceCollectionAsset* Collection : DaySequenceCollections)
	{
		if (Collection)
		{
			Objects.Add(Collection);
		}
	}
	Super::GetReferencedContentObjects(Objects);
	return true;
}

void ADaySequenceActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	auto ReinitializeActor = [this](bool bReinitializeRootSequence = false)
	{
		// Do not reinitialize actor for PIE sessions.
		if (UWorld* World = GetWorld(); World && World->IsGameWorld())
		{
			return;
		}
		if (bReinitializeRootSequence)
		{
			RootSequence = nullptr;
		}
		UpdateRootSequenceOnTickState.bUpdate = true;
		SubSections.Empty();
	};
	
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ADaySequenceActor, TimeOfDayPreview))
	{
		// Force the change to ensure it is broadcast to clients.
		SetTimeOfDayPreview(GetTimeOfDayPreview());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADaySequenceActor, DayLength))
	{
		SetDayLength(GetDayLength());

		// RootSequence playback range is computed using DayLength, so reconstruct RootSequence to reflect this change.
		ReinitializeActor(true);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADaySequenceActor, TimePerCycle))
	{
		SetTimePerCycle(GetTimePerCycle());

		// RootSequence playback range is computed using TimePerCycle, so reconstruct RootSequence to reflect this change.
		ReinitializeActor(true);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADaySequenceActor, InitialTimeOfDay))
	{
		SetInitialTimeOfDay(GetInitialTimeOfDay());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADaySequenceActor, DaySequenceCollections))
	{
		ReinitializeActor();
	}
	else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified)
	{
		// This handles undo/redo transactions.
		ReinitializeActor();
	}

	if (CameraModifierManager)
	{
		CameraModifierManager->ResetEditorPreview();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ADaySequenceActor::ConditionalSetTimeOfDayPreview(float InHours)
{
	// Wrap the input hours using day length
	InHours = FMath::Frac(InHours / GetDayLength()) * GetDayLength(); 
	const FDaySequenceTime NewTimeOfDayPreview = FDaySequenceTime::FromHours(InHours);
	if (NewTimeOfDayPreview != TimeOfDayPreview)
	{
		TimeOfDayPreview = NewTimeOfDayPreview;
		OnTimeOfDayPreviewChangedEvent.Broadcast(InHours);
		OnTimeOfDayPreviewChanged.Broadcast(InHours);
	}
}

#endif //WITH_EDITOR

float ADaySequenceActor::GetTimeOfDayPreview() const
{
#if WITH_EDITOR
	return TimeOfDayPreview.ToHours();
#else
	return 0.0f;
#endif
}

void ADaySequenceActor::SetTimeOfDayPreview(float InHours)
{
#if WITH_EDITOR
	// Wrap the input hours using day length
	InHours = FMath::Frac(InHours / GetDayLength()) * GetDayLength(); 
	const FDaySequenceTime NewTimeOfDayPreview = FDaySequenceTime::FromHours(InHours);
	TimeOfDayPreview = NewTimeOfDayPreview;
	OnTimeOfDayPreviewChangedEvent.Broadcast(InHours);
	OnTimeOfDayPreviewChanged.Broadcast(InHours);
#endif
}

void ADaySequenceActor::InitializePlayer()
{
	using namespace UE::DaySequence;

	InitializeRootSequence();

	if (GetWorld()->IsGameWorld())
	{
		if (SequencePlayer)
		{
			SequencePlayer->SetPlaybackClient(this);
			SequencePlayer->Initialize(RootSequence, this, GetPlaybackSettings(RootSequence));
		}

		if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
		{
			Player->OnPlay.AddUniqueDynamic(this, &ADaySequenceActor::StopDaySequenceUpdateTimer);
			Player->OnPause.AddUniqueDynamic(this, &ADaySequenceActor::StartDaySequenceUpdateTimer);
		}
	}
}

void ADaySequenceActor::InitializeRootSequence()
{
	using namespace UE::DaySequence;
	
	CSV_SCOPED_TIMING_STAT(DaySequence, InitializeRootSequence);

	if (IsTemplate())
	{
		return;
	}
	
	ensureMsgf(!SequencePlayer || !SequencePlayer->IsValid(), TEXT("InitializeRootSequence called but the sequence player has already been initialized."));
	
#if WITH_EDITOR
	// Do not generate the root sequence during cook.
	if (IsRunningCookCommandlet())
	{
		return;
	}
#endif

	OnPreRootSequenceChanged.Broadcast();

#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	UWorld* World = GetWorld();
	const bool bInEditorWorld = World && World->WorldType == EWorldType::Editor;
	
	if (!RootSequence || !bInEditorWorld)
	{
#endif
		RootSequence = NewObject<UDaySequence>(this, NAME_None, RF_Transient);
		RootSequence->Initialize(RF_Transient);
		RootSequence->SetSequenceFlags(EMovieSceneSequenceFlags::Volatile);

		SetRootSequencePlaybackRange();
#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	}
#endif

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	for (TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry> Entry : SubSectionDebugEntries)
	{
		UnregisterDebugEntry(Entry, ShowDebug_SubSequenceCategory);
	}
	
	SubSectionDebugEntries.Empty();
#endif
	
	if (GetNetMode() == NM_DedicatedServer)
	{
		return;
	}
	
#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	if (bInEditorWorld)
	{
		FSubSectionPreserveMap SectionsToPreserve;
		for (const UMovieSceneTrack* Track : RootSequence->GetMovieScene()->GetTracks())
		{
			if (const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections(); Sections.Num() > 0)
			{
				// There is an assumption of 1 section per track. If that assumption is not respected, root sequence reconstruction will likely break.
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Sections[0]))
				{
					SectionsToPreserve.Add(SubSection, false);
				}
			}
		}
		
		// This returns true if we need to do a full reinit and false if we can skip init
		if (MarkDaySequences(&SectionsToPreserve))
		{
			InitializeDaySequences();
			OnPostInitializeDaySequences.Broadcast(nullptr);
		}
		else
		{
			OnPostInitializeDaySequences.Broadcast(&SectionsToPreserve);
		}

		// Remove all unmarked sections
		for (const TPair<UMovieSceneSubSection*, bool>& Section : SectionsToPreserve)
		{
			if (!Section.Value)
			{
				BindingResolveFunctions.Remove(Section.Key->GetSequenceID());
				
#if WITH_EDITOR
				OnSubSectionRemovedEvent.Broadcast(Section.Key);
#endif
				
				UMovieSceneTrack* Track = Section.Key->GetTypedOuter<UMovieSceneTrack>();
				UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();
	
				check(Track && MovieScene);
	
				MovieScene->RemoveTrack(*Track);
				MovieScene->MarkAsChanged();
			}
		}
	}
	else
	{
		BindingResolveFunctions.Empty();
		InitializeDaySequences();
		OnPostInitializeDaySequences.Broadcast(nullptr);
	}
#else
	BindingResolveFunctions.Empty();
	InitializeDaySequences();
	OnPostInitializeDaySequences.Broadcast(nullptr);
#endif

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (!IsDebugCategoryRegistered(ShowDebug_SubSequenceCategory))
	{
		RegisterDebugCategory(ShowDebug_SubSequenceCategory,  OnShowDebugInfoDrawFunction);
	}

	for (TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry> Entry : SubSectionDebugEntries)
	{
		RegisterDebugEntry(Entry, ShowDebug_SubSequenceCategory);
	}
#endif
	
	OnPostRootSequenceChanged.Broadcast();
}

void ADaySequenceActor::SetRootSequencePlaybackRange()
{
	if (UMovieScene* RootMovieScene = RootSequence->GetMovieScene())
	{
		// Compute RootDuration so that the frame range as viewed by the user is [0, 100 * DayLengthHours].
		// For example, if DayLength is 24:00:00, frame range is [0, 2400].
		
		const FFrameRate TickDisplayRatio = RootMovieScene->GetTickResolution() / RootMovieScene->GetDisplayRate();
		const int32 HoursToFrames = 100 * TickDisplayRatio.Numerator / TickDisplayRatio.Denominator;
		const int32 RootDuration = FMath::Min(HoursToFrames * DayLength.ToHours(), 100000000);
		
		RootMovieScene->SetPlaybackRange(0, RootDuration);
#if WITH_EDITOR
		RootMovieScene->SetPlaybackRangeLocked(true);
#endif
		
		RootMovieScene->MarkAsChanged();
	}
}

float ADaySequenceActor::GetBasePlayRate() const
{
	if (!RootSequence)
	{
		return 1.f;
	}
	
	const float PlayerDurationSeconds = GetBaseDuration() * FDaySequenceTime::SecondsPerHour;
	return PlayerDurationSeconds / TimePerCycle.ToSeconds();
}

float ADaySequenceActor::GetBaseDuration() const
{
	if (!RootSequence)
	{
		return TimePerCycle.ToHours();
	}

	const UMovieScene* MovieScene = RootSequence->GetMovieScene();
	const float BaseDurationSeconds = MovieScene->GetTickResolution().AsSeconds(MovieScene->GetPlaybackRange().GetUpperBoundValue());

	return BaseDurationSeconds / FDaySequenceTime::SecondsPerHour;
}

#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
bool ADaySequenceActor::MarkDaySequences(FSubSectionPreserveMap* SectionsToPreserve)
{
	bool bReinit = true;

	if (SectionsToPreserve)
	{
		// Mark all subsections we have recorded for keep in the root sequence
		// This is a fast path we take only if all of our subsections are in the root sequence
		for (TWeakObjectPtr<UMovieSceneSubSection> SubSection : SubSections)
		{
			if (const UMovieSceneSubSection* StrongSubSection = SubSection.Get())
			{
				if (bool* SectionToPreserveFlag = SectionsToPreserve->Find(StrongSubSection))
				{
					*SectionToPreserveFlag = true;
					bReinit = false;
				}
				else
				{
					// If we have a subsection that is not in the root sequence, break and reinit completely
					bReinit = true;
					break;
				}
			}
		}
	}

	if (bReinit)
	{
		if (SectionsToPreserve)
		{
			// Mark all sections associated with this modifier for delete before we do a full reinit
			for (TWeakObjectPtr<UMovieSceneSubSection> SubSection : SubSections)
			{
				if (const UMovieSceneSubSection* StrongSubSection = SubSection.Get())
				{
					if (bool* SectionToPreserveFlag = SectionsToPreserve->Find(StrongSubSection))
					{
						*SectionToPreserveFlag = false;
					}
				}
			}
		}
	}

	return bReinit;
}
#endif

void ADaySequenceActor::InitializeDaySequences()
{
	SubSections.Empty();
	
	for (UDaySequenceCollectionAsset* Collection : DaySequenceCollections)
	{
		if (!Collection)
		{
			continue;
		}
		
		for (const FDaySequenceCollectionEntry& Entry : Collection->DaySequences)
		{
			InitializeDaySequence(Entry);
		}

		for (TInstancedStruct<FProceduralDaySequence>& ProceduralDaySequence : Collection->ProceduralDaySequences)
		{
			if (!ProceduralDaySequence.IsValid())
			{
				continue;
			}
			
			FProceduralDaySequence& ProceduralSequence = ProceduralDaySequence.GetMutable<FProceduralDaySequence>();
 
			if (UDaySequence* Sequence = ProceduralSequence.GetSequence(this))
			{
				FDaySequenceCollectionEntry TempEntry(Sequence);
				TempEntry.Conditions = ProceduralSequence.Conditions;
  
				InitializeDaySequence(TempEntry);
			}
		}
	}
}

UMovieSceneSubSection* ADaySequenceActor::InitializeDaySequence(const FDaySequenceCollectionEntry& SequenceAsset)
{
	if (!RootSequence || !SequenceAsset.Sequence)
	{
		return nullptr;
	}

	UMovieScene* RootMovieScene = RootSequence->GetMovieScene();
	UDaySequenceTrack* SubTrack = RootMovieScene->AddTrack<UDaySequenceTrack>();
	UMovieSceneSubSection* SubSection = nullptr;

#if WITH_EDITORONLY_DATA
	if (UMovieScene* SequenceMovieScene = SequenceAsset.Sequence->GetMovieScene())
	{
		SequenceMovieScene->SetReadOnly(SequenceAsset.Sequence->GetPackage()->bIsCookedForEditor);
	}
#endif
	
	if (SubTrack)
	{
		SubTrack->ClearFlags(RF_Transactional);
		SubTrack->SetFlags(RF_Transient);

		// Add the subsequence section with an arbitrary duration. It will be
		// normalized in UpdateSubSectionTimeScale.
		const int32 RootDuration = RootMovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
		SubSection = SubTrack->AddSequence(SequenceAsset.Sequence, 0, RootDuration);

		if (SubSection)
		{
			UpdateSubSectionTimeScale(SubSection);
			SubSection->Parameters.HierarchicalBias = SequenceAsset.BiasOffset + Bias;
			SubSection->Parameters.Flags            = EMovieSceneSubSectionFlags::OverrideRestoreState;

			const TFunction<void(void)> SetSubTrackMuteStateConditional = [this, SubSection, Conditions = SequenceAsset.Conditions.Conditions]()
			{
				if (!IsValidChecked(this) || !IsValid(SubSection))
				{
					return;
				}

				SubSection->SetIsLocked(false);
				// Begin SubSection mutation:
				
				constexpr bool bInitialMuteState = false;
				if (const bool bActive = !EvaluateSequenceConditions(bInitialMuteState, Conditions); SubSection->IsActive() != bActive)
				{
					SubSection->MarkAsChanged();
					SubSection->SetIsActive(bActive);
				}

				SubSection->SetIsLocked(true);
			};

			const TFunction<void(void)> SetSubTrackMuteStateUnconditional = [this, SubSection]()
			{
				if (!IsValidChecked(this) || !IsValid(SubSection))
				{
					return;
				}

				SubSection->SetIsLocked(false);
				// Begin SubSection mutation:
				
				if (!SubSection->IsActive())
				{
					SubSection->MarkAsChanged();
					SubSection->SetIsActive(true);
				}
				
				SubSection->SetIsLocked(true);
			};
			
			const TFunction<void(void)>& SetSubTrackMuteState = SequenceAsset.Conditions.Conditions.Num() == 0 ? SetSubTrackMuteStateUnconditional : SetSubTrackMuteStateConditional;
			
			// Initialize mute state and set up the condition callbacks to dynamically update mute state.
			SetSubTrackMuteState();
			OnInvalidateMuteStates.AddWeakLambda(SubSection, SetSubTrackMuteState);
			BindToConditionCallbacks(SubSection, SequenceAsset.Conditions.Conditions, [this]() { InvalidateMuteStates(); });

			SubSections.Add(SubSection);
		}
		else
		{
			UE_LOGF(LogDaySequence, Warning, "Failed to create UMovieSceneSubSection in ADaySequenceActor::InitializeDaySequence");
		}
	}
	else
	{
		UE_LOGF(LogDaySequence, Warning, "Failed to create UDaySequenceTrack in ADaySequenceActor::InitializeDaySequence");
	}

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (SubSection)
	{
        TSharedPtr<TMap<FString, FString>> DebugData = MakeShared<TMap<FString, FString>>();
        SubSectionDebugEntries.Emplace(MakeShared<UE::DaySequence::FDaySequenceDebugEntry>(
        [this](){ return true; },
        [this, DebugData, SubSection]()
        {
        	if (IsValid(SubSection))
        	{
        		(*DebugData).FindOrAdd("Name") = GetPathName(GetOwner() ? GetOwner()->GetOuter() : nullptr);
        		(*DebugData).FindOrAdd("Authority") = HasAuthority() ? "True" : "False";
        		(*DebugData).FindOrAdd("Sequence Name") = SubSection->GetSequence() ? SubSection->GetSequence()->GetFName().ToString() : "None";
        		(*DebugData).FindOrAdd("Mute State") = SubSection->IsActive() ? "Active" : "Muted";
        		(*DebugData).FindOrAdd("Hierarchical Bias") = FString::Printf(TEXT("%d"), SubSection->Parameters.HierarchicalBias);
        	}
        	return DebugData;
        },
        [SubSection]()
        {
			return IsValid(SubSection) ? SubSection->Parameters.HierarchicalBias : 0;
        }));
	}
#endif

	return SubSection;
}

void ADaySequenceActor::UpdateSubSectionTimeScale(UMovieSceneSubSection* InSubSection) const
{
	if (!InSubSection || !RootSequence)
	{
		return;
	}
	
	const UMovieSceneSequence* Sequence = InSubSection->GetSequence();
	if (!Sequence)
	{
		return;
	}
	
	// Compute outer duration from subsequence asset.
	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FQualifiedFrameTime InnerDuration = FQualifiedFrameTime(
		UE::MovieScene::DiscreteSize(MovieScene->GetPlaybackRange()),
		TickResolution);

	const UDaySequenceTrack* SubTrack = InSubSection->GetTypedOuter<UDaySequenceTrack>();
	const FFrameRate OuterFrameRate = SubTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
	const int32      OuterDuration  = InnerDuration.ConvertTo(OuterFrameRate).FrameNumber.Value;

	// Set the subsequence section to span the full day cycle range and
	// normalize playback by setting TimeScale on the section.
	const UMovieScene* RootMovieScene = RootSequence->GetMovieScene();
	const int32 RootDuration = RootMovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
	const bool bSubsectionWasLocked = InSubSection->IsLocked();
	InSubSection->SetIsLocked(false);
	InSubSection->MarkAsChanged();
	InSubSection->Parameters.TimeScale = (float)OuterDuration / (float)RootDuration;
	InSubSection->SetRange(RootMovieScene->GetPlaybackRange());
	InSubSection->SetIsLocked(bSubsectionWasLocked);
}

void ADaySequenceActor::OnSequencePlayerUpdate(const UDaySequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime)
{
	CSV_SCOPED_TIMING_STAT(DaySequence, OnSequencePlayerUpdate);
	
	auto FrameTimeToDayHours = [this](const FFrameTime& Time)
	{
		const UMovieScene* MovieScene = RootSequence->GetMovieScene();
		const FFrameRate FrameRate = MovieScene->GetDisplayRate();
		const double CurrentTimeSeconds = FQualifiedFrameTime(Time, FrameRate).AsSeconds();
		const float PlayerDurationSeconds = GetBaseDuration() * FDaySequenceTime::SecondsPerHour;
		const float SequenceRatio = CurrentTimeSeconds / PlayerDurationSeconds;
		return DayLength.ToHours() * SequenceRatio;
	};
	const float CurrentHours = FrameTimeToDayHours(CurrentTime);
	const float PreviousHours = FrameTimeToDayHours(PreviousTime);
	SequencePlayerUpdated(CurrentHours, PreviousHours);

	if (IsPlaying())
	{
		OnDaySequenceUpdate.Broadcast();
	}
}

void ADaySequenceActor::SequencePlayerUpdated(float CurrentTime, float PreviousTime)
{
}

void ADaySequenceActor::StartDaySequenceUpdateTimer()
{
	if (GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	if (const UWorld* World = GetWorld())
	{
		FTimerManagerTimerParameters TimerParameters;
		TimerParameters.bLoop = true;
		TimerParameters.bMaxOncePerFrame = true;

		World->GetTimerManager().SetTimer(DaySequenceUpdateTimerHandle, FTimerDelegate::CreateWeakLambda(this, [this]()
		{
			OnDaySequenceUpdate.Broadcast();
		}), FMath::Max(SequenceUpdateInterval, UE_SMALL_NUMBER), TimerParameters);
	}
}

void ADaySequenceActor::StopDaySequenceUpdateTimer()
{
	if (GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	if (const UWorld* World = GetWorld())
	{
		World->GetTimerManager().ClearTimer(DaySequenceUpdateTimerHandle);
	}
}

#if WITH_EDITORONLY_DATA
bool ADaySequenceActor::GetOverrideInitialTimeOfDay() const
{
	return bOverrideInitialTimeOfDay;
}

void ADaySequenceActor::SetOverrideInitialTimeOfDay(bool bNewOverrideInitialTimeOfDay)
{
	SetOverrideInitialTimeOfDay(bNewOverrideInitialTimeOfDay, GetTimeOfDayPreview());
}

void ADaySequenceActor::SetOverrideInitialTimeOfDay(bool bNewOverrideInitialTimeOfDay, float OverrideInitialTimeOfDay)
{
	bool bNeedsBroadcast = false;

	if (bOverrideInitialTimeOfDay != bNewOverrideInitialTimeOfDay)
	{
		bOverrideInitialTimeOfDay = bNewOverrideInitialTimeOfDay;
		bNeedsBroadcast = true;
	}

	if (!FMath::IsNearlyEqual(OverrideInitialTimeOfDay, GetTimeOfDayPreview()))
	{
		SetTimeOfDayPreview(OverrideInitialTimeOfDay);
		bNeedsBroadcast = true;
	}

	if (bNeedsBroadcast)
	{
		OnOverrideInitialTimeOfDayChanged.Broadcast(bOverrideInitialTimeOfDay, OverrideInitialTimeOfDay);
	}
}

bool ADaySequenceActor::GetOverrideRunDayCycle() const
{
	return bOverrideRunDayCycle;
}

void ADaySequenceActor::SetOverrideRunDayCycle(bool bNewOverrideRunDayCycle)
{
	if (bOverrideRunDayCycle != bNewOverrideRunDayCycle)
	{
		bOverrideRunDayCycle = bNewOverrideRunDayCycle;
		OnOverrideRunDayCycleChanged.Broadcast(bOverrideRunDayCycle);
	}
}
#endif

bool ADaySequenceActor::HasValidRootSequence() const
{
	return IsValid(RootSequence);
}

bool ADaySequenceActor::RootSequenceHasValidSections() const
{
	if (IsValid(RootSequence))
	{
		if (const UMovieScene* RootMovieScene = RootSequence->GetMovieScene())
		{
			for (const UMovieSceneSection* Section : RootMovieScene->GetAllSections())
			{
				if (IsValid(Section))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

void ADaySequenceActor::InvalidateMuteStates() const
{
	OnInvalidateMuteStates.Broadcast();
}

void ADaySequenceActor::WarpEvaluationRange(FMovieSceneEvaluationRange& InOutRange) const
{
	if (!RootSequence)
	{
		return;
	}
	
	const FFrameRate TickRate   = InOutRange.GetFrameRate();
	const UMovieScene* MovieScene = RootSequence->GetMovieScene();
	const float DayCycleSeconds = MovieScene->GetTickResolution().AsSeconds(MovieScene->GetPlaybackRange().GetUpperBoundValue());
	const float DayLengthHours  = DayLength.ToHours();

	TRange<FFrameTime> Range = InOutRange.GetRange();

	// Auto bounds checking
	auto TrySetBounds = [&Range, MovieScene](FFrameTime LowerBound, FFrameTime UpperBound)
	{
		// Warp the lower bound if possible
		if (Range.GetLowerBound().IsClosed())
		{
			// Set the lower bound value while retaining the inclusivity
			Range.SetLowerBoundValue(LowerBound);
		}

		// Warp the upper bound if possible
		if (Range.GetUpperBound().IsClosed())
		{
			// Set the upper bound value while retaining the inclusivity.
			// We are clamping to [0, PlaybackRangeUpperBound - 1] because our playback range has an exclusive upper bound.
			Range.SetUpperBoundValue(FMath::Clamp(UpperBound, FFrameTime(0), FFrameTime(MovieScene->GetPlaybackRange().GetUpperBoundValue() - 1)));
		}
	};

	// Warp with static time if necessary
	if (HasStaticTimeOfDay())
	{
		const float StaticTimeInGameHours = GetStaticTimeOfDay();

		// The approach here:
		// 1) converts static time to normalized time (StaticTimeInGameHours / DayLengthHours)
		// 2) converts normalized time to sequencer time (X * DayCycleSeconds)
		// 3) converts sequencer time to a frame time (X * TickRate)

		const FFrameTime LowerBound = (StaticTimeInGameHours * DayCycleSeconds / DayLengthHours) * TickRate;
		const FFrameTime UpperBound = (StaticTimeInGameHours * DayCycleSeconds / DayLengthHours) * TickRate;

		TrySetBounds(LowerBound, UpperBound);
	}

	// Warp with curve
	if (bUseInterpCurve && DayInterpCurve && !bForceDisableDayInterpCurve)
	{
		const float LowerBoundTimeSeconds = static_cast<float>(Range.GetLowerBoundValue() / TickRate);
		float LowerBoundTimeInHours = DayLengthHours * LowerBoundTimeSeconds / DayCycleSeconds;
		LowerBoundTimeInHours = DayInterpCurve->FloatCurve.Eval(LowerBoundTimeInHours, LowerBoundTimeInHours);
		const FFrameTime LowerBound = (LowerBoundTimeInHours * DayCycleSeconds / DayLengthHours) * TickRate;

		const float UpperBoundTimeSeconds = static_cast<float>(Range.GetUpperBoundValue() / TickRate);
		float UpperBoundTimeInHours = DayLengthHours * UpperBoundTimeSeconds / DayCycleSeconds;
		UpperBoundTimeInHours = DayInterpCurve->FloatCurve.Eval(UpperBoundTimeInHours, UpperBoundTimeInHours);
		const FFrameTime UpperBound = (UpperBoundTimeInHours * DayCycleSeconds / DayLengthHours) * TickRate;

		TrySetBounds(LowerBound, UpperBound);
	}
	
	InOutRange.ResetRange(Range);
}

bool ADaySequenceActor::GetRunDayCycle() const
{
	return bRunDayCycle;
}

void ADaySequenceActor::SetRunDayCycle(bool bNewRunDayCycle)
{
	bRunDayCycle = bNewRunDayCycle;
}

float ADaySequenceActor::GetDayLength() const
{
	return DayLength.ToHours();
}

void ADaySequenceActor::SetDayLength(float InHours)
{
	// Set min day length to 1 second.
	InHours = FMath::Max(InHours, FDaySequenceTime::FromSeconds(1.f).ToHours());
	DayLength = FDaySequenceTime::FromHours(InHours);
}

float ADaySequenceActor::GetTimePerCycle() const
{
	return TimePerCycle.ToHours();
}

void ADaySequenceActor::SetTimePerCycle(float InHours)
{
	// Set min cycle length to 1 second.
	InHours = FMath::Max(InHours, FDaySequenceTime::FromSeconds(1.f).ToHours());
	TimePerCycle = FDaySequenceTime::FromHours(InHours);
}

void ADaySequenceActor::SetPlayRate_Implementation(float NewRate)
{
	if (UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		// we are given a rate w.r.t. DSA time, not player time.
		Player->SetPlayRate(NewRate * GetBasePlayRate());
	}
}
	
float ADaySequenceActor::GetPlayRate() const
{
	// Game world
	if (const UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		return Player->GetPlayRate() / GetBasePlayRate();
	}

	// Fallback
	return 1.f;
}

void ADaySequenceActor::Multicast_SetTimePerCycle_Implementation(float InHours)
{
	using namespace UE::DaySequence;

	const ADaySequenceActor* CDO = GetClass()->GetDefaultObject<ADaySequenceActor>();
	const float DesiredPlayRate = CDO->TimePerCycle.ToHours() / InHours;

	// We are explicitly calling the implementation because this function itself is already NetMulticast.
	SetPlayRate_Implementation(DesiredPlayRate);
}

float ADaySequenceActor::GetInitialTimeOfDay() const
{
	return InitialTimeOfDay.ToHours();
}

void ADaySequenceActor::SetInitialTimeOfDay(float InHours)
{
	InHours = FMath::Clamp(InHours, 0.0f, GetDayLength()); 
	InitialTimeOfDay = FDaySequenceTime::FromHours(InHours);
}

float ADaySequenceActor::GetTimeOfDay() const
{
	float Result;
	UWorld* World = GetWorld();
	const UDaySequencePlayer* Player = GetSequencePlayerInternal();
	if (HasValidRootSequence() && Player && World && World->IsGameWorld())
	{
		const FQualifiedFrameTime CurrentFrameTime = Player->GetCurrentTime();
		const double CurrentTimeSeconds = CurrentFrameTime.AsSeconds();
		const float PlayerDurationSeconds = GetBaseDuration() * FDaySequenceTime::SecondsPerHour;
		const float SequenceRatio = CurrentTimeSeconds / PlayerDurationSeconds;
		Result = DayLength.ToHours() * SequenceRatio;
	}
	else
	{
#if WITH_EDITOR
		Result = GetTimeOfDayPreview();
#else
		Result = GetInitialTimeOfDay();
#endif
	}
	return Result;
}

float ADaySequenceActor::GetApparentTimeOfDay() const
{
	using namespace UE::DaySequence;

	// Query both FixedTime and LocalFixedTime channels for Apparent time of day.
	constexpr EStaticTimeChannel ApparentChannels = EStaticTimeChannel::FixedTime | EStaticTimeChannel::LocalFixedTime;
	float Time;
	if (StaticTimeManager && StaticTimeManager->HasStaticTime(ApparentChannels))
	{
		Time = StaticTimeManager->GetStaticTime(GetTimeOfDay(), DayLength.ToHours(), ApparentChannels);
	}
	else
	{
		Time = GetTimeOfDay();
	}
	return GetInterpCurveTimeOfDay(Time);
}

float ADaySequenceActor::GetTargetApparentTimeOfDay() const
{
	using namespace UE::DaySequence;
	
	constexpr EStaticTimeChannel ApparentChannels = EStaticTimeChannel::FixedTime | EStaticTimeChannel::LocalFixedTime;
	float Time;
	if (StaticTimeManager && StaticTimeManager->HasStaticTime(ApparentChannels))
	{
		Time = StaticTimeManager->GetTargetStaticTime(GetTimeOfDay(), ApparentChannels);
	}
	else
	{
		Time = GetTimeOfDay();
	}
	return Time;
}

bool ADaySequenceActor::SetTimeOfDay(float InHours)
{
#if !UE_BUILD_SHIPPING
	if (UE::DaySequence::CVarDumpStackTraceOnSetTimeOfDay.GetValueOnAnyThread())
	{
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
	}
#endif
	
	// Only set time of day if we have a valid playing day sequence.
	UWorld* World = GetWorld();
	UDaySequencePlayer* Player = GetSequencePlayerInternal();
	if (HasValidRootSequence() && Player && World && World->IsGameWorld())
	{
		const UMovieScene* MovieScene = RootSequence->GetMovieScene();
		const FFrameRate FrameRate = MovieScene->GetDisplayRate();
		
		// Convert the day time to sequence time
		const float PlayerDurationSeconds = GetBaseDuration() * FDaySequenceTime::SecondsPerHour;
		const float DayLengthRatio = FMath::Frac(InHours / GetDayLength());
		const float DayCycleSeconds = PlayerDurationSeconds * DayLengthRatio;

		// Update the playback position of the sequence.
		FDaySequencePlaybackParams PlaybackParams;
		PlaybackParams.Frame = FrameRate.AsFrameTime(DayCycleSeconds); 
		PlaybackParams.UpdateMethod = EUpdatePositionMethod::Play;
		Player->SetPlaybackPosition(PlaybackParams);
		return true;
	}
	return false;
}

bool ADaySequenceActor::HasStaticTimeOfDay() const
{
	return StaticTimeManager && StaticTimeManager->HasStaticTime();
}

float ADaySequenceActor::GetStaticTimeOfDay() const
{
	using namespace UE::DaySequence;
	
	float Time;
	if (HasStaticTimeOfDay())
	{
		Time = StaticTimeManager->GetStaticTime(GetTimeOfDay(), DayLength.ToHours(), EStaticTimeChannel::FixedTime);
	}
	else
	{
		Time = GetTimeOfDay();
	}
	return Time;
}

void ADaySequenceActor::RegisterStaticTimeContributor(const UE::DaySequence::FStaticTimeContributor& NewContributor) const
{
	if (!StaticTimeManager)
	{
		return;
	}
	
	StaticTimeManager->AddStaticTimeContributor(NewContributor);
}

void ADaySequenceActor::UnregisterStaticTimeContributor(const UObject* InUserObject) const
{
	if (!StaticTimeManager)
	{
		return;
	}
	
	StaticTimeManager->RemoveStaticTimeContributor(InUserObject);
}

float ADaySequenceActor::GetInterpCurveTimeOfDay(float Time) const
{
	const bool bUseCurve = bUseInterpCurve && DayInterpCurve && !bForceDisableDayInterpCurve;
	return bUseCurve ? DayInterpCurve->FloatCurve.Eval(Time, Time) : Time;
}

void ADaySequenceActor::RegisterBindingResolveFunction(FMovieSceneSequenceID SequenceID, FGuid Guid, TFunction<bool(TArray<UObject*, TInlineAllocator<1>>&)> InFunction)
{
	UnregisterBindingResolveFunction(SequenceID, Guid);

	BindingResolveFunctions.FindOrAdd(SequenceID).FindOrAdd(Guid) = InFunction;
}

void ADaySequenceActor::UnregisterBindingResolveFunction(FMovieSceneSequenceID SequenceID, FGuid Guid)
{
	if (auto* SubsequenceResolveFunctions = BindingResolveFunctions.Find(SequenceID))
	{
		if (Guid.IsValid())
		{
			SubsequenceResolveFunctions->Remove(Guid);
		}

		// If no more functions associated with this sequence ID or no binding guid was specified, just remove the entry for this sequence.
		if (SubsequenceResolveFunctions->Num() == 0 || !Guid.IsValid())
		{
			SubsequenceResolveFunctions = nullptr;	// we are invalidating this ptr
			BindingResolveFunctions.Remove(SequenceID);
		}
	}
}

void ADaySequenceActor::Play()
{
	if (!bRunDayCycle)
	{
		UE_LOGF(LogDaySequence, Warning, "Cannot begin/resume playback while bRunDayCycle is false.");
		return;
	}

	UDaySequencePlayer* Player = GetSequencePlayerInternal();
	UWorld* World = GetWorld();
	if (HasValidRootSequence() && Player && World && World->IsGameWorld())
	{
		// Always ensure play is looping.
		Player->PlayLooping();
	}
}

void ADaySequenceActor::Pause()
{
	UDaySequencePlayer* Player = GetSequencePlayerInternal();
	UWorld* World = GetWorld();
	if (HasValidRootSequence() && Player && World && World->IsGameWorld())
	{
		Player->Pause();
	}
}

bool ADaySequenceActor::IsPlaying() const
{
	if (const UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		return Player->IsPlaying();
	}

	return false;
}

bool ADaySequenceActor::IsPaused() const
{
	if (const UDaySequencePlayer* Player = GetSequencePlayerInternal())
	{
		return Player->IsPaused();
	}

	return false;
}

UDaySequence* ADaySequenceActor::GetRootSequence() const
{
	return RootSequence;
}

void ADaySequenceActor::UpdateRootSequence(EUpdateRootSequenceMode Mode)
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	
	// Set up a callback for when UObjects are modified so we can catch changes to our 
	FCoreUObjectDelegates::OnObjectModified.AddWeakLambda(this, [this](UObject* InObject)
	{
		if (IsValid(InObject) && InObject->IsA(UDaySequenceCollectionAsset::StaticClass()) && DaySequenceCollections.Contains(InObject))
		{
			// We update on next tick because calling UpdateScalabilitySequence here
			// is too early (our collection doesn't have the changes that triggered this invocation).
			UpdateRootSequenceOnTick(EUpdateRootSequenceMode::Reinitialize);
		}
	});
#endif

	if (EnumHasAnyFlags(Mode, EUpdateRootSequenceMode::Reinitialize))
	{
		// Force rebuild the root sequence.
		SubSections.Empty();
		RootSequence = nullptr;
	}
	InitializeRootSequence();
}

#if WITH_EDITOR
void ADaySequenceActor::UpdateRootSequenceOnTick(EUpdateRootSequenceMode Mode)
{
	UpdateRootSequenceOnTickState.bUpdate = true;
	UpdateRootSequenceOnTickState.Mode |= Mode;
}
#endif


/**
 * Compute a PlaybackSettings object for the given DaySequence with a fixed 1.0x PlayRate.
 *
 * This method is used to workaround the issue where a non-1.0x PlayRate causes issues with
 * sequence playback replication.
 */
FMovieSceneSequencePlaybackSettings ADaySequenceActor::GetPlaybackSettings(const UDaySequence* Sequence) const
{
	FMovieSceneSequencePlaybackSettings Settings;
	Settings.bAutoPlay = true;
	Settings.LoopCount.Value = -1; // Loop indefinitely
	Settings.bDisableCameraCuts = true;
	Settings.PlayRate = 1.0f;
	Settings.StartTime = 0.0f;

	// User configurable update interval.
	Settings.TickInterval.TickIntervalSeconds = SequenceUpdateInterval;
	
	// Set explicit frame budget based on the cvar
	Settings.TickInterval.EvaluationBudgetMicroseconds = UE::DaySequence::GFrameBudgetMicroseconds;

	// Tick interval is configured above
	Settings.bInheritTickIntervalFromOwner = false;

	if (Sequence && Sequence->GetMovieScene())
	{
		Settings.PlayRate = GetBasePlayRate();

#if WITH_EDITORONLY_DATA
		const float InitialHours = bOverrideInitialTimeOfDay ? GetTimeOfDayPreview() : GetInitialTimeOfDay();
#else
		const float InitialHours = GetInitialTimeOfDay();
#endif
		
		const float StartRatio = FMath::Frac(InitialHours / GetDayLength());
		const float PlayerDurationSeconds = GetBaseDuration() * FDaySequenceTime::SecondsPerHour;
		const float DayCycleSeconds = PlayerDurationSeconds * StartRatio;
		Settings.StartTime = DayCycleSeconds;
	}
	return Settings;
}

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
bool ADaySequenceActor::IsDebugCategoryRegistered(const FName& Category) const
{
	return DebugEntries.Find(Category) != nullptr;
}

void ADaySequenceActor::RegisterDebugCategory(const FName& Category, UE::DaySequence::FDebugCategoryDrawFunction DrawFunction)
{
	using namespace UE::DaySequence;
	
	if (TPair<FDebugEntryArray, FDebugCategoryDrawFunction>* CategoryHandle = DebugEntries.Find(Category))
	{
		ensureMsgf(false, TEXT("Category \"%s\" is already registered with this DaySequenceActor."), *Category.ToString());
		return;
	}

	DebugEntries.Add(Category, TPair<FDebugEntryArray, FDebugCategoryDrawFunction>(FDebugEntryArray(), DrawFunction));
}

void ADaySequenceActor::RegisterDebugEntry(const TWeakPtr<UE::DaySequence::FDaySequenceDebugEntry>& DebugEntry, const FName& Category)
{
	using namespace UE::DaySequence;
	
	if (TPair<FDebugEntryArray, FDebugCategoryDrawFunction>* CategoryHandle = DebugEntries.Find(Category))
	{
		FDebugEntryArray& Entries = CategoryHandle->Key;
		Entries.AddUnique(DebugEntry);
	}
	else
	{
		ensureMsgf(false, TEXT("Category \"%s\" is not registered with this DaySequenceActor."), *Category.ToString());
	}
}

void ADaySequenceActor::UnregisterDebugEntry(const TWeakPtr<UE::DaySequence::FDaySequenceDebugEntry>& DebugEntry, const FName& Category)
{
	using namespace UE::DaySequence;
	
	if (TPair<FDebugEntryArray, FDebugCategoryDrawFunction>* CategoryValue = DebugEntries.Find(Category))
	{
		FDebugEntryArray& Entries = CategoryValue->Key;
		Entries.Remove(DebugEntry);
	}
	else
	{
		ensureMsgf(false, TEXT("Category \"%s\" is not registered with this DaySequenceActor."), *Category.ToString());
	}
}

void ADaySequenceActor::OnShowDebugInfoDrawFunction(UCanvas* Canvas, TArray<TSharedPtr<TMap<FString, FString>>>& Entries, const FString& Category)
{
	using namespace UE::DaySequence;

	// Early out if this table will be empty (occurs if the Entry->ShowCondition() condition above never evaluates to true)
	if (Entries.Num() == 0)
	{
		return;
	}
	
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	const UFont* Font = GEngine->GetSmallFont();
	DisplayDebugManager.SetFont(Font);
	
	// Used for padding table entries. Char count isn't sufficient as these fonts aren't monospace so we must
	// compute the number of spaces when we need to pad as the width in pixels of the area we need to pad divided
	// by the width of a single space character and then pad appropriately
	float SpaceCharacterWidth = 0.f;
	float SpaceCharacterHeight = 0.f;
	Font->GetCharSize(' ', SpaceCharacterWidth, SpaceCharacterHeight);

	const FString FieldSeparator = "    |    ";
	const int32 FieldSeparatorSize = Font->GetStringSize(&FieldSeparator[0]);
	
	// Determine column widths based on the largest value to be printed in each column (including column labels)
	// Also keep track of the running total row length which is simply the sum of the columns lengths
	TMap<FString, int32> LongestPropertyValues;
	int32 TotalExpectedRowLength = 0;
	for (const auto& Entry : Entries)
	{
		for (const auto& Cell : *Entry)
		{
			const int32 NewPropertySize = Font->GetStringSize(&Cell.Value[0]);

			if (int32* ExistingPropertySize = LongestPropertyValues.Find(Cell.Key))
			{
				// Update existing property's maximum known size
				
				const int32 IncreaseInPropertySize = FMath::Max(NewPropertySize - *ExistingPropertySize, 0);
				*ExistingPropertySize += IncreaseInPropertySize;
				TotalExpectedRowLength += IncreaseInPropertySize;

				// This is the more obvious way to do this but makes tracking the running total more annoying
				// *ExistingPropertySize = FMath::Max(*ExistingPropertySize, NewPropertySize);
			}
			else
			{
				// Add new length entry that is max(StringSize, PropertyNameStringSize) because the property
				// names are printed in their own row and should factor into the column widths

				const int32 IncreaseInRowLength = FMath::Max(NewPropertySize, Font->GetStringSize(&Cell.Key[0]));
				LongestPropertyValues.Add(Cell.Key, IncreaseInRowLength);
				TotalExpectedRowLength += IncreaseInRowLength;
			}
			
			int32& Length = LongestPropertyValues.FindOrAdd(Cell.Key, NewPropertySize);
			Length = FMath::Max(Length, NewPropertySize);
		}
	}
	// If we have N columns then there are N - 1 separators between them.
	TotalExpectedRowLength += (LongestPropertyValues.Num() - 1) * FieldSeparatorSize;

	/* BEGIN DRAWING HERE */

	auto PadToCenterString = [Font, SpaceCharacterWidth](const FString& StringToCenter, const int32 DesiredStringLength)
	{
		const int32 StringToCenterSize = Font->GetStringSize(&StringToCenter[0]);
		const int32 PadSpaceCount = FMath::CeilToInt((DesiredStringLength - StringToCenterSize) / SpaceCharacterWidth);
		const int32 PadLeftCount = PadSpaceCount / 2;
		const int32 PadRightCount = PadSpaceCount / 2 + PadSpaceCount % 2;

		// LeftPad/RightPad will attempt to pad with space such that the returned string's length is ChCount (the single parameter)
		// It is NOT padding by ChCount characters and it does not modify the internal string, it returns a copy.
		// So first we pad to the left by the current string size + the spaces we want on the left then we pad the returned
		// string by the original string size + the desired left spaces + the desired right spaces.
		return StringToCenter.LeftPad(StringToCenter.Len() + PadLeftCount).RightPad(StringToCenter.Len() + PadLeftCount + PadRightCount);
	};

	// Print some new lines to clearly separate this table from any previous data.
	// A better way to do this would be to using the DisplayDebugManagers SetYPos() function but it isn't exported.
	constexpr int32 NumInitialLineSkips = 3;
	for (int Line = 0; Line < NumInitialLineSkips; ++Line)
	{
		DisplayDebugManager.DrawString("");
	}
	
	// Print header text
	{
		const FString RowText = FieldSeparator + PadToCenterString("Category: " + Category, TotalExpectedRowLength) + FieldSeparator;
		DisplayDebugManager.SetDrawColor(FColor::Yellow);
		DisplayDebugManager.DrawString(RowText);
	}
	
	// Print column labels
	{
		FString RowText = FieldSeparator;
		
		for (const auto& Property : LongestPropertyValues)
		{
			RowText += PadToCenterString(Property.Key, Property.Value) + FieldSeparator;
		}
	
		DisplayDebugManager.DrawString(RowText);
	}

	// Print column values
	DisplayDebugManager.SetDrawColor(FColor::White);
	for (const auto& Entry : Entries)
	{
		TMap<FString, FString>& EntryData = *Entry;
		FString RowText = FieldSeparator;

		for (const auto& Property : LongestPropertyValues)
		{
			RowText += PadToCenterString(EntryData.Find(Property.Key) ? EntryData[Property.Key] : "None", Property.Value) + FieldSeparator;
		}

		DisplayDebugManager.DrawString(RowText);
	}
}

void ADaySequenceActor::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	using namespace UE::DaySequence;
	
	if (!Canvas || !GEngine || GetNetMode() == NM_DedicatedServer || !HUD || GetWorld() != HUD->GetWorld())
	{
		return;
	}

#if WITH_EDITOR
	// Necessary because we actually get called from a static delegate, so this can be called for editor & PIE actor which obfuscates the data.
	if (const UWorld* World = GetWorld(); World && World->WorldType == EWorldType::Editor)
	{
		return;
	}
#endif

	DebugEntries = DebugEntries.FilterByPredicate([](const TPair<FName, TPair<FDebugEntryArray, FDebugCategoryDrawFunction>>& Category) {
		return !Category.Value.Key.IsEmpty();
	});

	auto RemoveStaleEntriesAndGetPinnedArray = [](FDebugEntryArray& InWeakArray, TArray<TSharedPtr<TMap<FString, FString>>>& OutSharedArray)
	{
		// Shouldn't reduce existing capacity, so in theory the total number
		// of allocations here will be <= the size of the largest array in DebugEntries
		OutSharedArray.Reset();
		
		auto RemovePredicate = [](const TWeakPtr<FDaySequenceDebugEntry>& Entry){ return !Entry.IsValid(); };
		InWeakArray.RemoveAll(RemovePredicate);

		auto TransformPredicate = [](const TWeakPtr<FDaySequenceDebugEntry>& Entry){ return Entry.Pin()->ShowCondition(); };
		auto TransformFunction = [](const TWeakPtr<FDaySequenceDebugEntry>& Entry){ return Entry.Pin()->GetData(); };
		Algo::TransformIf(InWeakArray, OutSharedArray, TransformPredicate, TransformFunction);
	};

	class FDebugEntriesLessThan
	{
	public:

		bool operator()(const TWeakPtr<FDaySequenceDebugEntry>& LHS, const TWeakPtr<FDaySequenceDebugEntry>& RHS) const
		{
			const TSharedPtr<FDaySequenceDebugEntry> PinnedLHS = LHS.Pin();
			const TSharedPtr<FDaySequenceDebugEntry> PinnedRHS = RHS.Pin();

			if (PinnedLHS && PinnedRHS)
			{
				return (PinnedLHS->GetNumericValue ? PinnedLHS->GetNumericValue() : 0) < (PinnedRHS->GetNumericValue ? PinnedRHS->GetNumericValue() : 0);
			}

			// LHS is expired & RHS is valid, expired elements are always less than valid elements.
			if (PinnedLHS)
			{
				return false;
			}

			// LHS is valid & RHS is expired, expired elements are always less than valid elements.
			if (PinnedRHS)
			{
				return true;
			}

			// If both are expired, then we consider the values equal (so LHS is not less than RHS).
			return false;
		}
	};
	
	TArray<TSharedPtr<TMap<FString, FString>>> EntriesToDraw;
	for (TPair<FName, TPair<FDebugEntryArray, FDebugCategoryDrawFunction>>& Category : DebugEntries)
	{
		// Print this category if it is individually enabled via "showdebug Category.Key" OR if
		// the general DaySequenceCategory is enabled via "showdebug DaySequence"
		if (HUD->ShouldDisplayDebug(ShowDebug_GeneralCategory) || HUD->ShouldDisplayDebug(Category.Key))
		{
			FDebugEntryArray& Entries = Category.Value.Key;
			FDebugCategoryDrawFunction& DrawFunction = Category.Value.Value;

			Entries.StableSort(FDebugEntriesLessThan());
			
			RemoveStaleEntriesAndGetPinnedArray(Entries, EntriesToDraw);

			DrawFunction(Canvas, EntriesToDraw, Category.Key.ToString());
		}
	}
}
#endif

TObjectPtr<UDaySequenceConditionTag> ADaySequenceActor::GetOrInstantiateConditionTag(const TSubclassOf<UDaySequenceConditionTag>& ConditionClass)
{
	/**
	 * If ConditionClass is nullptr, ConditionClass is not a child of UDaySequenceConditionTag, or we are PostLoading, catch here.
	 * Note: *ConditionClass returns a UClass* (does not dereference pointers), so *ConditionClass == nullptr is a null check.
	 * Early out if we are PostLoading because we can't safely call UDaySequenceConditionTag::Initialize (we will reinitialize sequences soon after this).
	 */
	if (*ConditionClass == nullptr || !ConditionClass->IsChildOf(UDaySequenceConditionTag::StaticClass()) || FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return nullptr;
	}

	// Instantiate condition if necessary
	if (!TrackConditionMap.Contains(*ConditionClass) || (TrackConditionMap.Contains(*ConditionClass) && !IsValid(TrackConditionMap[*ConditionClass])))
	{
		TrackConditionMap.Remove(*ConditionClass);
		TrackConditionMap.Add(*ConditionClass, NewObject<UDaySequenceConditionTag>(this, ConditionClass));
		UDaySequenceConditionTag* ConditionTag = TrackConditionMap[*ConditionClass];
		
		UWorld* World = GetWorld();

		switch (TrackConditionMap[*ConditionClass]->GetInitializationPhase())
		{
		case EDaySequenceConditionTagInitializationPhase::Immediate:
			ConditionTag->Initialize();
			break;

		case EDaySequenceConditionTagInitializationPhase::GameStateReady:
			if (World)
			{
				World->GameStateSetEvent.AddWeakLambda(ConditionTag, [this, ConditionTag](AGameStateBase*)
				{
					ConditionTag->Initialize();
					InvalidateMuteStates();
				});

				if (World->GetGameState())
				{
					ConditionTag->Initialize();
				}
			}
			break;
			
		default:
			break;
		}
	}

	return TrackConditionMap[*ConditionClass];
}

bool ADaySequenceActor::EvaluateSequenceConditions(bool bInitialMuteState, const FDaySequenceConditionSet::FConditionValueMap& InConditions)
{
	bool bMuted = bInitialMuteState;
	
	for (const TTuple<TSubclassOf<UDaySequenceConditionTag>, bool>& Element : InConditions)
	{
		const TSubclassOf<UDaySequenceConditionTag>& ConditionClass = Element.Key;
		const bool& ExpectedValue = Element.Value;
		
		if (const TObjectPtr<UDaySequenceConditionTag> ConditionInstance = GetOrInstantiateConditionTag(ConditionClass))
		{
			// If ConditionInstance evaluates to ExpectedValue, bMuted is unchanged. Otherwise, bMuted is set to true.
			bMuted |= ConditionInstance->Evaluate() != ExpectedValue;
		}
	}

	return bMuted;
}

void ADaySequenceActor::BindToConditionCallbacks(UObject* LifetimeObject, const FDaySequenceConditionSet::FConditionValueMap& InConditions, const TFunction<void(void)>& InFunction)
{
	for (const TTuple<TSubclassOf<UDaySequenceConditionTag>, bool>& Element : InConditions)
	{
		const TSubclassOf<UDaySequenceConditionTag>& ConditionClass = Element.Key;
		
		if (const TObjectPtr<UDaySequenceConditionTag> ConditionInstance = GetOrInstantiateConditionTag(ConditionClass))
		{
			// Invoke InFunction when ConditionInstance's OnConditionValueChanged delegate is broadcast if LifetimeObject is still valid.
			ConditionInstance->GetOnConditionValueChanged().AddWeakLambda(LifetimeObject, InFunction);
		}
	}
}

#if WITH_EDITOR
void ADaySequenceActor::HandleConditionReinstanced(const FCoreUObjectDelegates::FReplacementObjectMap& OldToNewInstanceMap)
{
	for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
	{
		if (Pair.Key->IsTemplate())
		{
			continue;
		}

		// Casting too much here? Second one might be unnecessary
		if (UDaySequenceConditionTag* OldTag = Cast<UDaySequenceConditionTag>(Pair.Key))
		{
			if (UDaySequenceConditionTag* NewTag = Cast<UDaySequenceConditionTag>(Pair.Value))
			{
				NewTag->GetOnConditionValueChanged() = OldTag->GetOnConditionValueChanged();
			}
		}
	}
}
#endif