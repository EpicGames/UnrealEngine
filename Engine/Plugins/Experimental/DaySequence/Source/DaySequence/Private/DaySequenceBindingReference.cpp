// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceBindingReference.h"
#include "DaySequenceActor.h"

#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "UnrealEngine.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceBindingReference)

FDaySequenceBindingReference::FDaySequenceBindingReference(UObject* InObject, UObject* InContext)
{
	check(InContext && InObject);

	// Special case for the global ADaySequenceActor that is provided as the context
	if (InObject == InContext)
	{
		return;
	}

	// InContext should always be an actor - either the ADaySequenceActor being played back,
	// or a parent actor if this is a component reference.
	const bool bIsComponentBinding = InObject->IsIn(InContext);
	if (bIsComponentBinding)
	{
		ObjectPath = InObject->GetPathName(InContext);
	}
	else
	{
		FString FullPath = InObject->GetPathName();

#if WITH_EDITORONLY_DATA
		UPackage* ObjectPackage = InObject->GetOutermost();

		if (ensure(ObjectPackage))
		{
			// If this is being set from PIE we need to remove the pie prefix and point to the editor object
			if (ObjectPackage->GetPIEInstanceID() != INDEX_NONE)
			{
				FString PIEPrefix = FString::Printf(PLAYWORLD_PACKAGE_PREFIX TEXT("_%d_"), ObjectPackage->GetPIEInstanceID());
				FullPath.ReplaceInline(*PIEPrefix, TEXT(""));
			}
		}
#endif

		ExternalObjectPath = FSoftObjectPath(FullPath);
	}
}

FDaySequenceBindingReference FDaySequenceBindingReference::DefaultRootBinding()
{
	return SpecializedBinding(EDaySequenceBindingReferenceSpecialization::Root);
}

FDaySequenceBindingReference FDaySequenceBindingReference::SpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization)
{
	FDaySequenceBindingReference Default;
	Default.Specialization = Specialization;
	return Default;
}

UObject* FDaySequenceBindingReference::Resolve(UObject* InContext) const
{
	if (InContext == nullptr)
	{
		return nullptr;
	}

	// Context must always be an AActor - either the ADaySequenceActor playing back, or a parent actor (for component bindings)
	check(InContext->IsA<AActor>());

	// This branch is where specialized bindings are resolved (or where resolution is deferred).
	if (ExternalObjectPath.IsNull() && ObjectPath.Len() == 0)
	{
		switch(Specialization)
		{
		case EDaySequenceBindingReferenceSpecialization::None:	// legacy path here, new references should never be None if the object paths are empty.
		case EDaySequenceBindingReferenceSpecialization::Root:
		{
			ADaySequenceActor* DaySequenceActor = Cast<ADaySequenceActor>(InContext);
			ensureMsgf(DaySequenceActor, TEXT("Failed to locate the currently playing back day sequence actor - was InContext provided incorrectly?"));
			return DaySequenceActor;
		}

		// resolution handled elsewhere, not enough info
		default:
		case EDaySequenceBindingReferenceSpecialization::CameraModifier:
			return nullptr;
		}
	}

	if (!ExternalObjectPath.IsNull())
	{
		// If we have an external object path we must be bound to an Actor
		FSoftObjectPath TempPath = ExternalObjectPath.ToSoftObjectPath();

		// Soft Object Paths don't follow asset redirectors when attempting to call ResolveObject or TryLoad.
		// We want to follow the asset redirector so that maps that have been renamed (from Untitled to their first asset name)
		// properly resolve. This fixes Possessable bindings losing their references the first time you save a map.
		TempPath.PreSavePath();

	#if WITH_EDITORONLY_DATA
		// Sequencer is explicit about providing a resolution context for its bindings. We never want to resolve to objects
		// with a different PIE instance ID, even if the current callstack is being executed inside a different GPlayInEditorID
		// scope. Since ResolveObject will always call FixupForPIE in editor based on GPlayInEditorID, we always override the current
		// GPlayInEditorID to be the current PIE instance of the provided context.
		const int32 ContextPlayInEditorID = InContext ? InContext->GetOutermost()->GetPIEInstanceID() : INDEX_NONE;
		FTemporaryPlayInEditorIDOverride PIEGuard(ContextPlayInEditorID);
	#endif

		UObject* ResultObject = TempPath.ResolveObject();
		return ResultObject;
	}
	else
	{
		// Component binding - look up the object path within the provided context object
		if (UE::IsSavingPackage(nullptr) || IsGarbageCollecting())
		{
			return nullptr;
		}

		return FindObject<UObject>(InContext, *ObjectPath, EFindObjectFlags::None);
	}
}

bool FDaySequenceBindingReference::operator==(const FDaySequenceBindingReference& Other) const
{
	return ExternalObjectPath == Other.ExternalObjectPath && ObjectPath == Other.ObjectPath;
}

#if WITH_EDITORONLY_DATA
void FDaySequenceBindingReference::PerformLegacyFixup()
{
	// Reset bindings that point to the global ADaySequenceActor
	const UClass* ClassPtr = ObjectClass_DEPRECATED.IsPending() ? ObjectClass_DEPRECATED.LoadSynchronous() : ObjectClass_DEPRECATED.Get();
	if (ClassPtr && ClassPtr->IsChildOf(ADaySequenceActor::StaticClass()))
	{
		ObjectPath.Empty();
		ExternalObjectPath.Reset();
	}

	// Fixup the specialization of empty bindings that have no specialization. This is the case for Root DSA bindings created before the specialization system.
	if (ExternalObjectPath.IsNull() && ObjectPath.Len() == 0 && Specialization == EDaySequenceBindingReferenceSpecialization::None)
	{
		Specialization = EDaySequenceBindingReferenceSpecialization::Root;
	}
}
#endif

bool FDaySequenceBindingReferences::HasBinding(const FGuid& ObjectId) const
{
	return BindingIdToReferences.Contains(ObjectId) || AnimSequenceInstances.Contains(ObjectId);
}

void FDaySequenceBindingReferences::AddBinding(const FGuid& ObjectId, UObject* InObject, UObject* InContext)
{
	if (InObject->IsA<UAnimInstance>())
	{
		AnimSequenceInstances.Add(ObjectId);
	}
	else
	{
		BindingIdToReferences.FindOrAdd(ObjectId).References.Emplace(InObject, InContext);
	}
}

void FDaySequenceBindingReferences::AddDefaultBinding(const FGuid& ObjectId)
{
	AddSpecializedBinding(ObjectId, EDaySequenceBindingReferenceSpecialization::Root);
}

void FDaySequenceBindingReferences::AddSpecializedBinding(const FGuid& ObjectId, EDaySequenceBindingReferenceSpecialization Specialization)
{
	BindingIdToReferences.FindOrAdd(ObjectId).References.Emplace(FDaySequenceBindingReference::SpecializedBinding(Specialization));
	SpecializedReferenceToGuid.FindOrAdd(Specialization) = ObjectId;
	GuidToSpecializedReference.FindOrAdd(ObjectId) = Specialization;
}

FGuid FDaySequenceBindingReferences::FindSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization) const
{
	const FGuid* BindingGuid = SpecializedReferenceToGuid.Find(Specialization);
	return BindingGuid ? *BindingGuid : FGuid();
}

void FDaySequenceBindingReferences::RemoveBinding(const FGuid& ObjectId)
{
	BindingIdToReferences.Remove(ObjectId);
	AnimSequenceInstances.Remove(ObjectId);

	// If this Guid is a specialized binding, we need to remove our tracking info.
	if (const EDaySequenceBindingReferenceSpecialization* Specialization = GuidToSpecializedReference.Find(ObjectId))
	{
		SpecializedReferenceToGuid.Remove(*Specialization);
		GuidToSpecializedReference.Remove(ObjectId);	// warning: Specialization is probably a bad ptr after this removal.
	}
}

void FDaySequenceBindingReferences::RemoveObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext)
{
	FDaySequenceBindingReferenceArray* ReferenceArray = BindingIdToReferences.Find(ObjectId);
	if (!ReferenceArray)
	{
		return;
	}

	for (int32 ReferenceIndex = 0; ReferenceIndex < ReferenceArray->References.Num(); )
	{
		UObject* ResolvedObject = ReferenceArray->References[ReferenceIndex].Resolve(InContext);

		if (InObjects.Contains(ResolvedObject))
		{
			ReferenceArray->References.RemoveAt(ReferenceIndex);
		}
		else
		{
			++ReferenceIndex;
		}
	}
}

void FDaySequenceBindingReferences::RemoveInvalidObjects(const FGuid& ObjectId, UObject* InContext)
{
	FDaySequenceBindingReferenceArray* ReferenceArray = BindingIdToReferences.Find(ObjectId);
	if (!ReferenceArray)
	{
		return;
	}

	for (int32 ReferenceIndex = 0; ReferenceIndex < ReferenceArray->References.Num(); )
	{
		UObject* ResolvedObject = ReferenceArray->References[ReferenceIndex].Resolve(InContext);

		if (!IsValid(ResolvedObject))
		{
			ReferenceArray->References.RemoveAt(ReferenceIndex);
		}
		else
		{
			++ReferenceIndex;
		}
	}
}

void FDaySequenceBindingReferences::ResolveBinding(const FGuid& ObjectId, UObject* InContext, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (const FDaySequenceBindingReferenceArray* ReferenceArray = BindingIdToReferences.Find(ObjectId))
	{
		for (const FDaySequenceBindingReference& Reference : ReferenceArray->References)
		{
			UObject* ResolvedObject = Reference.Resolve(InContext);
			if (ResolvedObject && ResolvedObject->GetWorld())
			{
				OutObjects.Add(ResolvedObject);
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InContext))
	{
		// If the object ID exists in the AnimSequenceInstances set, then this binding relates to an anim instance on a skeletal mesh component
		if (SkeletalMeshComponent && AnimSequenceInstances.Contains(ObjectId) && SkeletalMeshComponent->GetAnimInstance())
		{
			OutObjects.Add(SkeletalMeshComponent->GetAnimInstance());
		}
	}
}

void FDaySequenceBindingReferences::RemoveInvalidBindings(const TSet<FGuid>& ValidBindingIDs)
{
	for (auto It = BindingIdToReferences.CreateIterator(); It; ++It)
	{
		if (!ValidBindingIDs.Contains(It->Key))
		{
			It.RemoveCurrent();
		}
	}
}

#if WITH_EDITORONLY_DATA
void FDaySequenceBindingReferences::PerformLegacyFixup()
{
	for (TPair<FGuid, FDaySequenceBindingReferenceArray>& Pair : BindingIdToReferences)
	{
		for (FDaySequenceBindingReference& Reference : Pair.Value.References)
		{
			const EDaySequenceBindingReferenceSpecialization PreFixupSpecialization = Reference.GetSpecialization();
			Reference.PerformLegacyFixup();
			const EDaySequenceBindingReferenceSpecialization PostFixupSpecialization = Reference.GetSpecialization();

			// Make sure we track already created bindings that are fixed up to be specialized (this is the case for old Root DSA bindings).
			if (PreFixupSpecialization == EDaySequenceBindingReferenceSpecialization::None && PreFixupSpecialization != PostFixupSpecialization)
			{
				SpecializedReferenceToGuid.FindOrAdd(PostFixupSpecialization) = Pair.Key;
				GuidToSpecializedReference.FindOrAdd(Pair.Key) = PostFixupSpecialization;
			}
		}
	}
}
#endif
