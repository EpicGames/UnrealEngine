// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceCameraModifier.h"

#include "Components/PostProcessComponent.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceCameraModifier)

UCameraModifier* UDaySequenceCameraModifierManager::GetCameraModifier(APlayerController* InPC)
{
	if (InPC)
	{
		if (const TWeakObjectPtr<UCameraModifier>* Modifier = CameraModifiers.Find(InPC))
		{
			return Modifier->Get();
		}

		if (APlayerCameraManager* CameraManager = InPC->PlayerCameraManager)
		{
			UCameraModifier* NewModifier = CameraManager->AddNewCameraModifier(UDaySequenceCameraModifier::StaticClass());
			CameraModifiers.Emplace(InPC, NewModifier);
			return NewModifier;
		}
	}
#if WITH_EDITOR
	else
	{
		return GetEditorCameraModifier();
	}
#endif
	
	return nullptr;
}

#if WITH_EDITOR
UCameraModifier* UDaySequenceCameraModifierManager::GetEditorCameraModifier()
{
	if (const UWorld* World = GetWorld(); World && World->WorldType != EWorldType::Editor)
	{
		return nullptr;
	}
	
	if (!EditorCameraModifier)
	{
		EditorCameraModifier = NewObject<UDaySequenceCameraModifier>(this, "EditorCameraModifier", RF_Transient);
	}

	if (!EditorCameraModiferPreview)
	{
		EditorCameraModiferPreview = NewObject<UPostProcessComponent>(GetOuter(), "EditorCameraModifierPreview", RF_Transient);
		EditorCameraModiferPreview->RegisterComponent();
		EditorCameraModiferPreview->AttachToComponent(EditorCameraModiferPreview->GetOwner()->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	}

	return EditorCameraModifier;
}

void UDaySequenceCameraModifierManager::UpdateEditorPreview() const
{
	if (EditorCameraModifier && EditorCameraModiferPreview)
	{
		EditorCameraModiferPreview->Settings = EditorCameraModifier->GetSettings();
	}
}
#endif