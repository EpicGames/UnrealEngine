// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceCheatManagerExtension.h"

#include "DaySequenceActor.h"
#include "DaySequenceSubsystem.h"
#include "Engine/World.h"
#include "GameFramework/CheatManagerDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceCheatManagerExtension)

UDaySequenceCheatManagerExtension::UDaySequenceCheatManagerExtension()
{
#if UE_WITH_CHEAT_MANAGER
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UCheatManager::RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate::CreateLambda(
		[](UCheatManager* CheatManager)
		{
			if (const UWorld* World = CheatManager->GetWorld())
			{
				if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
				{
					DaySequenceSubsystem->OnCheatManagerCreated(CheatManager);
				}
			}
		}));
	}
#endif
}

void UDaySequenceCheatManagerExtension::SetTimeOfDay(float NewTimeOfDay) const
{
	if (ADaySequenceActor* DaySequenceActor = GetDaySequenceActor())
	{
		DaySequenceActor->SetTimeOfDay(NewTimeOfDay);
	}
}

void UDaySequenceCheatManagerExtension::SetTimeOfDaySpeed(float NewTimeOfDaySpeedMultiplier) const
{
	if (ADaySequenceActor* DaySequenceActor = GetDaySequenceActor())
	{
		if (NewTimeOfDaySpeedMultiplier < 0.f)
		{
			return;
		}

		if (NewTimeOfDaySpeedMultiplier == 0.f)
		{
			DaySequenceActor->Pause();
		}
		else
		{
			DaySequenceActor->SetPlayRate(NewTimeOfDaySpeedMultiplier);
			DaySequenceActor->Play();
		}
	}
}

ADaySequenceActor* UDaySequenceCheatManagerExtension::GetDaySequenceActor() const
{
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			return DaySequenceSubsystem->GetDaySequenceActor();
		}
	}

	return nullptr;
}
