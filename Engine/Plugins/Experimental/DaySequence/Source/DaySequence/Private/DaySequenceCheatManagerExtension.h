// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/CheatManager.h"

#include "DaySequenceCheatManagerExtension.generated.h"

class ADaySequenceActor;

/** Cheats related to DaySequence */
UCLASS(NotBlueprintable)
class UDaySequenceCheatManagerExtension final : public UCheatManagerExtension
{
	GENERATED_BODY()

public:
	UDaySequenceCheatManagerExtension();

	UFUNCTION(Exec)
	void SetTimeOfDay(float NewTimeOfDay) const;

	UFUNCTION(Exec)
	void SetTimeOfDaySpeed(float NewTimeOfDaySpeedMultiplier) const;

private:
	ADaySequenceActor* GetDaySequenceActor() const;
};