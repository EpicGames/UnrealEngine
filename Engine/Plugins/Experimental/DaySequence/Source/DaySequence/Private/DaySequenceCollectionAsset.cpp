// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceCollectionAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceCollectionAsset)

#define LOCTEXT_NAMESPACE "DaySequenceCollectionAsset"

// See TODO in DaySequenceCollectionAsset.h

#undef LOCTEXT_NAMESPACE