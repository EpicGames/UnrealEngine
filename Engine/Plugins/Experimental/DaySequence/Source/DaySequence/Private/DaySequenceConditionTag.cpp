// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceConditionTag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceConditionTag)

void UDaySequenceConditionTag::Initialize()
{
	SetupOnConditionValueChanged();
}

bool UDaySequenceConditionTag::Evaluate_Implementation() const
{
	return true;
}

FString UDaySequenceConditionTag::GetConditionName() const
{
	return ConditionName.IsEmpty() ? GetClass()->GetName() : ConditionName;
}

EDaySequenceConditionTagInitializationPhase UDaySequenceConditionTag::GetInitializationPhase() const
{
	return InitializationPhase;
}

UWorld* UDaySequenceConditionTag::GetWorld() const
{
	if (IsTemplate())
	{
		return nullptr;
	}

	return GetOuter()->GetWorld();
}

void UDaySequenceConditionTag::SetupOnConditionValueChanged_Implementation() const
{
}

void UDaySequenceConditionTag::BroadcastOnConditionValueChanged() const
{
	const bool bResult = Evaluate();
	if (!bCachedEvalResult.IsSet() || bCachedEvalResult.GetValue() != bResult)
	{
		bCachedEvalResult = bResult;
		OnConditionValueChanged.Broadcast();
	}
}
