// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceDirector.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieScenePlayback.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "IMovieScenePlayer.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceDirector)

UWorld* UDaySequenceDirector::GetWorld() const
{
	if (ULevel* OuterLevel = GetTypedOuter<ULevel>())
	{
		return OuterLevel->OwningWorld;
	}
	return GetTypedOuter<UWorld>();
}

FQualifiedFrameTime UDaySequenceDirector::GetRootSequenceTime() const
{
	using namespace UE::MovieScene;

	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = PlayerInterface->GetEvaluationTemplate();
		UMovieSceneSequence* RootSequence = EvaluationTemplate.GetRootSequence();
		const FSequenceInstance* RootSequenceInstance = EvaluationTemplate.FindInstance(MovieSceneSequenceID::Root);
		if (RootSequenceInstance && RootSequence)
		{
			// Put the qualified frame time into 'display' rate
			FFrameRate DisplayRate = RootSequence->GetMovieScene()->GetDisplayRate();
			FMovieSceneContext Context = RootSequenceInstance->GetContext();

			FFrameTime DisplayRateTime = ConvertFrameTime(Context.GetTime(), Context.GetFrameRate(), DisplayRate);
			return FQualifiedFrameTime(DisplayRateTime, DisplayRate);
		}
	}
	return FQualifiedFrameTime(0, FFrameRate());
}

FQualifiedFrameTime UDaySequenceDirector::GetCurrentTime() const
{
	using namespace UE::MovieScene;

	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = PlayerInterface->GetEvaluationTemplate();

		UMovieSceneSequence* SubSequence = EvaluationTemplate.GetSequence(FMovieSceneSequenceID(SubSequenceID));
		const FSequenceInstance* SequenceInstance = EvaluationTemplate.FindInstance(FMovieSceneSequenceID(SubSequenceID));
		if (SequenceInstance && SubSequence)
		{
			// Put the qualified frame time into 'display' rate
			FFrameRate DisplayRate = SubSequence->GetMovieScene()->GetDisplayRate();
			FMovieSceneContext Context = SequenceInstance->GetContext();

			FFrameTime DisplayRateTime = ConvertFrameTime(Context.GetTime(), Context.GetFrameRate(), DisplayRate);
			return FQualifiedFrameTime(DisplayRateTime, DisplayRate);
		}
	}
	return FQualifiedFrameTime(0, FFrameRate());
}


TArray<UObject*> UDaySequenceDirector::GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding)
{
	TArray<UObject*> Objects;

	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(FMovieSceneSequenceID(SubSequenceID), *PlayerInterface))
		{
			if (UObject* Object = WeakObject.Get())
			{
				Objects.Add(Object);
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return Objects;
}

UObject* UDaySequenceDirector::GetBoundObject(FMovieSceneObjectBindingID ObjectBinding)
{
	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(FMovieSceneSequenceID(SubSequenceID), *PlayerInterface))
		{
			if (UObject* Object = WeakObject.Get())
			{
				return Object;
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return nullptr;
}

TArray<AActor*> UDaySequenceDirector::GetBoundActors(FMovieSceneObjectBindingID ObjectBinding)
{
	TArray<AActor*> Actors;

	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(FMovieSceneSequenceID(SubSequenceID), *PlayerInterface))
		{
			if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
			{
				Actors.Add(Actor);
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return Actors;
}

AActor* UDaySequenceDirector::GetBoundActor(FMovieSceneObjectBindingID ObjectBinding)
{
	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(FMovieSceneSequenceID(SubSequenceID), *PlayerInterface))
		{
			if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
			{
				return Actor;
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return nullptr;
}

UMovieSceneSequence* UDaySequenceDirector::GetSequence()
{
	if (IMovieScenePlayer* PlayerInterface = IMovieScenePlayer::Get(static_cast<uint16>(MovieScenePlayerIndex)))
	{
		return PlayerInterface->GetEvaluationTemplate().GetSequence(FMovieSceneSequenceID(SubSequenceID));
	}
	else
	{		
		FFrame::KismetExecutionMessage(TEXT("No sequence player."), ELogVerbosity::Error);

		return nullptr;
	}
}


