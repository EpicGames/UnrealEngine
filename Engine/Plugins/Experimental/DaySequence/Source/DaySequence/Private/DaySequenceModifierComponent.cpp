// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceModifierComponent.h"

#include "DaySequence.h"
#include "DaySequenceCameraModifier.h"
#include "DaySequenceCollectionAsset.h"
#include "DaySequenceModule.h"
#include "DaySequencePlayer.h"
#include "DaySequenceStaticTime.h"
#include "DaySequenceSubsystem.h"
#include "DaySequenceTrack.h"

#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Camera/CameraModifier.h"
#include "Components/BoxComponent.h"
#include "Components/CapsuleComponent.h"
#include "Components/SphereComponent.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceModifierComponent)

#define LOCTEXT_NAMESPACE "DaySequenceModifierComponent"

namespace UE::DaySequence
{
	FVector GVolumePreviewLocation = FVector::ZeroVector;
	bool bIsSimulating = false;

	float ComputeBoxSignedDistance(const UBoxComponent* BoxComponent, const FVector& InWorldPosition)
	{
		const FTransform& ComponentTransform = BoxComponent->GetComponentTransform();

		FVector Point = ComponentTransform.InverseTransformPositionNoScale(InWorldPosition);
		FVector Box = BoxComponent->GetUnscaledBoxExtent() * ComponentTransform.GetScale3D();

		FVector Delta = Point.GetAbs() - Box;
		return FVector::Max(Delta, FVector::ZeroVector).Length() + FMath::Min(Delta.GetMax(), 0.0);
	}

	float ComputeSphereSignedDistance(const USphereComponent* SphereComponent, const FVector& InWorldPosition)
	{
		const FTransform& ComponentTransform = SphereComponent->GetComponentTransform();

		FVector Point = ComponentTransform.InverseTransformPositionNoScale(InWorldPosition);
		return Point.Length()-SphereComponent->GetScaledSphereRadius();
	}
	
	float ComputeCapsuleSignedDistance(const UCapsuleComponent* CapsuleComponent, const FVector& InWorldPosition)
	{
		// UCapsuleComponent::GetScaledCapsuleRadius() returns the min scaled X/Y axis for the radius
		// while the actual collision query uses the max scaled X/Y axis for the radius. We use Max here to match the collision.
		const FTransform& ComponentTransform = CapsuleComponent->GetComponentTransform();
		const FVector&    ComponentScale     = ComponentTransform.GetScale3D();

		FVector Point = ComponentTransform.InverseTransformPositionNoScale(InWorldPosition);
		const double CapsuleHalfHeight = CapsuleComponent->GetScaledCapsuleHalfHeight_WithoutHemisphere();
		const double CapsuleRadius = CapsuleComponent->GetUnscaledCapsuleRadius() * FMath::Max(ComponentScale.X, ComponentScale.Y);

		Point.Z = FMath::Max(FMath::Abs(Point.Z) - CapsuleHalfHeight, 0.0);
		return Point.Length() - CapsuleRadius;
	}

	float ComputeSignedDistance(const UShapeComponent* ShapeComponent, const FVector& InWorldPosition)
	{
		if (!ShapeComponent)
		{
			return UE_MAX_FLT;
		}
		
		if (const UBoxComponent* BoxComponent = Cast<UBoxComponent>(ShapeComponent))
		{
			return ComputeBoxSignedDistance(BoxComponent, InWorldPosition);
		}
		else if (const USphereComponent* SphereComponent = Cast<USphereComponent>(ShapeComponent))
		{
			return ComputeSphereSignedDistance(SphereComponent, InWorldPosition);
		}
		else if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(ShapeComponent))
		{
			return ComputeCapsuleSignedDistance(CapsuleComponent, InWorldPosition);
		}
		
		// @todo: unsupported shape?
		return (InWorldPosition - ShapeComponent->GetComponentLocation()).Length();
	}
	
	static TAutoConsoleVariable<bool> CVarModifierDisableWhenInvisible(
		TEXT("DaySequence.Modifier.DisableWhenInvisible"),
		false,
		TEXT("When true, day sequence modifier components will automatically disable when they are made invisible."),
		ECVF_Default
	);

	static TAutoConsoleVariable<bool> CVarModifierBlendPositionValidateTimestamp(
		TEXT("DaySequence.Modifier.BlendPositionValidateTimestamp"),
		true,
		TEXT("When true, day sequence modifier components will check the camera manager's timestamp for validity before returning a position."),
		ECVF_Default
	);
	
} // namespace UE::DaySequence

void UDaySequenceModifierEasingFunction::Initialize(EEasingFunctionType EasingType)
{
	if (UDaySequenceModifierComponent* Outer = Cast<UDaySequenceModifierComponent>(GetOuter()))
	{
		switch (EasingType)
		{
		case EEasingFunctionType::EaseIn:
			EvaluateImpl = [Outer](float)
			{
				return Outer->GetBlendWeight();
			};
			break;
	
		case EEasingFunctionType::EaseOut:
			EvaluateImpl = [Outer](float)
			{
				return 1.f - Outer->GetBlendWeight();
			};
			break;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Initialize called without a valid Outer!"));
		EvaluateImpl = [](float){ return 0.f; };
	}
}

float UDaySequenceModifierEasingFunction::Evaluate(float Interp) const
{
	return EvaluateImpl(Interp);
}

UDaySequenceModifierComponent::UDaySequenceModifierComponent(const FObjectInitializer& Init)
	: Super(Init)
{
	bIsComponentEnabled = true;
	bIsEnabled = false;
	bIgnoreBias = false;
	bPreview = true;
	bUseCollection = false;
	bSmoothBlending = false;
	bCachedExternalShapesInvalid = true;
	Bias = 1000;
	DayNightCycleTime = 12.f;
	DayNightCycle = EDayNightCycleMode::Default;
	Mode = EDaySequenceModifierMode::Volume;
	BlendPolicy = EDaySequenceModifierUserBlendPolicy::Minimum;
	BlendAmount = 100.f;
	UserBlendWeight = 1.f;
	InternalBlendWeight = 0.f;
	BlendTime = 0.f;
	TimedBlendingLastUpdated = 0.f;
	TimedBlendingStartWeight = 0.f;
	TimedBlendingTargetWeight = 0.f;
	TimedBlendingTargetWeightLastUpdated = 0.f;
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	DebugLevel = 0;
#endif
	
	PrimaryComponentTick.bCanEverTick = false;
	
	EasingFunction = CreateDefaultSubobject<UDaySequenceModifierEasingFunction>("EasingFunction", true);
}

#if WITH_EDITOR

void UDaySequenceModifierComponent::SetVolumePreviewLocation(const FVector& Location)
{
	UE::DaySequence::GVolumePreviewLocation = Location;
}

void UDaySequenceModifierComponent::SetIsSimulating(bool bInIsSimulating)
{
	UE::DaySequence::bIsSimulating = bInIsSimulating;
}

void UDaySequenceModifierComponent::UpdateEditorPreview(float DeltaTime)
{
	using namespace UE::DaySequence;

	if (bIsComponentEnabled && bPreview && IsRegistered() && !GetWorld()->IsGameWorld())
	{
		const float OldEffectiveBlendWeight = GetBlendWeight();
		UpdateInternalBlendWeight();
		
		if (GetBlendWeight() > UE_SMALL_NUMBER)
		{
			EnableModifier();
			
			// this compares effective blend weights, which is necessary in case user blend weight is changing while the internal blend weight remains constant.
			if (!FMath::IsNearlyEqual(OldEffectiveBlendWeight, GetBlendWeight()))
			{
				// If we're using a blend we have to mark active sections as changed
				// in order to force an update in-editor:
				
				for (TWeakObjectPtr<UMovieSceneSubSection> SubSection : SubSections)
				{
					UMovieSceneSubSection* StrongSubSection = SubSection.Get();
					if (StrongSubSection && StrongSubSection->IsActive())
					{
						StrongSubSection->MarkAsChanged();
						break;
					}
				}
			}
		}
		else
		{
			DisableModifier();
		}
	}
}

TStatId UDaySequenceModifierComponent::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UDaySequenceModifierComponent, STATGROUP_Tickables);
}

ETickableTickType UDaySequenceModifierComponent::GetTickableTickType() const
{
	UWorld* World = GetWorld();
	if (World && World->WorldType == EWorldType::Editor)
	{
		return ETickableTickType::Always;
	}
	return ETickableTickType::Never;
}

void UDaySequenceModifierComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDaySequenceModifierComponent, bPreview))
	{
		if (bPreview && !bIsEnabled)
		{
			EnableModifier();
		}
		else if (!bPreview && bIsEnabled)
		{
			DisableModifier();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDaySequenceModifierComponent, Mode))
	{
		EnableModifier();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDaySequenceModifierComponent, DayNightCycle))
	{
		// Force details panel changes to use our setter.
		SetDayNightCycle(DayNightCycle);
	}
}

#endif // WITH_EDITOR

void UDaySequenceModifierComponent::OnRegister()
{
	Super::OnRegister();

	bCachedExternalShapesInvalid = true;

#if WITH_EDITOR
	if (GIsEditor)
	{
		SetTickableTickType(GetTickableTickType());
	}
#endif
}

void UDaySequenceModifierComponent::OnUnregister()
{
	Super::OnUnregister();

	bCachedExternalShapesInvalid = true;
	
	DisableModifier();
	RemoveSubSequenceTrack();

#if WITH_EDITOR
	if (GIsEditor)
	{
		SetTickableTickType(ETickableTickType::Never);
	}
#endif
}

void UDaySequenceModifierComponent::DaySequenceUpdate()
{
	CSV_SCOPED_TIMING_STAT(DaySequence, SequencePlayerUpdated);

	bool bWantsOverride = false;

	if (!OverrideUpdateIntervalHandle && TargetActor)
	{
		if (IDaySequencePlayer* Player = TargetActor->GetSequencePlayer())
		{
			OverrideUpdateIntervalHandle = Player->GetOverrideUpdateIntervalHandle();
		}
	}
	
	if (bIsComponentEnabled)
	{
		// Force the expensive update
		UpdateInternalBlendWeight();
		
		if (GetBlendWeight() > UE_SMALL_NUMBER)
		{
			EnableModifier();
		}
		else
		{
			DisableModifier();
		}

		if (bIsEnabled && bSmoothBlending)
		{
			if (bForceSmoothBlending || FMath::IsWithin(GetBlendWeight(), 0.f + UE_KINDA_SMALL_NUMBER, 1.f - UE_KINDA_SMALL_NUMBER))
			{
				bWantsOverride = true;
			}
		}
	}
	
	if (OverrideUpdateIntervalHandle)
	{
		if (bWantsOverride)
		{
			OverrideUpdateIntervalHandle->StartOverriding();
		}
		else
		{
			OverrideUpdateIntervalHandle->StopOverriding();
		}
	}
}

void UDaySequenceModifierComponent::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Transfer DaySequenceCollection (deprecated) to DaySequenceCollections
	if (DaySequenceCollection)
	{
		DaySequenceCollections.Add(DaySequenceCollection);
		DaySequenceCollection = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDaySequenceModifierComponent::BeginPlay()
{
	Super::BeginPlay();
}

void UDaySequenceModifierComponent::EndPlay(EEndPlayReason::Type Reason)
{
	Super::EndPlay(Reason);
	
	RemoveSubSequenceTrack();
}

void UDaySequenceModifierComponent::BindToDaySequenceActor(ADaySequenceActor* DaySequenceActor)
{
	if (TargetActor == DaySequenceActor)
	{
		return;
	}

	bool bWasEnabled = bIsEnabled;
	UnbindFromDaySequenceActor();

	TargetActor = DaySequenceActor;

	if (bWasEnabled)
	{
		EnableModifier();
	}

	if (ensureMsgf(DaySequenceActor, TEXT("BindToDaySequenceActor called with a null Day Sequence Actor.")))
	{
		if (IDaySequencePlayer* Player = DaySequenceActor->GetSequencePlayer())
		{
			OverrideUpdateIntervalHandle = Player->GetOverrideUpdateIntervalHandle();
		}
		
		DaySequenceActor->GetOnPostInitializeDaySequences().AddUObject(this, static_cast<void (UDaySequenceModifierComponent::*)(ADaySequenceActor::FSubSectionPreserveMap*)>(&UDaySequenceModifierComponent::ReinitializeSubSequence));
		DaySequenceActor->GetOnDaySequenceUpdate().AddUObject(this, &UDaySequenceModifierComponent::DaySequenceUpdate);
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
		if (!DaySequenceActor->IsDebugCategoryRegistered(ShowDebug_ModifierCategory))
		{
			DaySequenceActor->RegisterDebugCategory(ShowDebug_ModifierCategory, TargetActor->OnShowDebugInfoDrawFunction);
		}
		
		DaySequenceActor->GetOnDebugLevelChanged().AddUObject(this, &UDaySequenceModifierComponent::OnDebugLevelChanged);
		DebugLevel = DaySequenceActor->GetDebugLevel();
	
		// This gets captured by a lambda below so should continue living
		TSharedPtr<TMap<FString, FString>> DebugData = MakeShared<TMap<FString, FString>>();
		DebugEntry = MakeShared<UE::DaySequence::FDaySequenceDebugEntry>(
		[this](){ return ShouldShowDebugInfo(); },
		[this, DebugData]()
		{
			(*DebugData).FindOrAdd("Name") = GetPathName(GetOwner() ? GetOwner()->GetOuter() : nullptr);
			(*DebugData).FindOrAdd("Component Enabled") = bIsComponentEnabled ? "True" : "False";
			(*DebugData).FindOrAdd("Modifier Enabled") = bIsEnabled ? "True" : "False";
			(*DebugData).FindOrAdd("Blend Weight") = FString::Printf(TEXT("%.5f"), GetBlendWeight());

			const APlayerController* BlendTarget = WeakBlendTarget.Get();
			(*DebugData).FindOrAdd("Blend Target" ) = BlendTarget ? BlendTarget->GetName() : "None";

			return DebugData;
		},
		[]() { return 0; });
		
		DaySequenceActor->RegisterDebugEntry(DebugEntry, ShowDebug_ModifierCategory);
#endif
	}
}

void UDaySequenceModifierComponent::UnbindFromDaySequenceActor()
{
	DisableModifier();
	RemoveSubSequenceTrack();

	if (OverrideUpdateIntervalHandle)
	{
		OverrideUpdateIntervalHandle.Reset();
	}
	
	if (TargetActor)
	{
		TargetActor->GetOnPostInitializeDaySequences().RemoveAll(this);
		TargetActor->GetOnDaySequenceUpdate().RemoveAll(this);
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
		TargetActor->GetOnDebugLevelChanged().RemoveAll(this);
		TargetActor->UnregisterDebugEntry(DebugEntry, ShowDebug_ModifierCategory);
		DebugEntry.Reset();
#endif
		TargetActor = nullptr;
	}
}

void UDaySequenceModifierComponent::RemoveSubSequenceTrack()
{
	auto RemoveSubTrack = [](const UMovieSceneSubSection* SubSection)
	{
		if (SubSection)
		{
			UMovieSceneTrack* Track = SubSection->GetTypedOuter<UMovieSceneTrack>();
			UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();

			check(Track && MovieScene);

			MovieScene->RemoveTrack(*Track);
			MovieScene->MarkAsChanged();
		}
	};
	
	for (const TWeakObjectPtr<UMovieSceneSubSection> WeakSubSection : SubSections)
	{
		const UMovieSceneSubSection* SubSection = WeakSubSection.Get();
		
#if WITH_EDITOR
		ADaySequenceActor::OnSubSectionRemovedEvent.Broadcast(SubSection);
#endif

		// When we untrack a subsection, we need to remove any associated resolve functions.
		if (TargetActor && SubSection)
		{
			TargetActor->UnregisterBindingResolveFunction(SubSection->GetSequenceID());
		}
		
		RemoveSubTrack(SubSection);
	}
	SubSections.Empty();

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (TargetActor)
	{
		for (TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry> Entry : SubSectionDebugEntries)
		{
			TargetActor->UnregisterDebugEntry(Entry, TargetActor->ShowDebug_SubSequenceCategory);
		}
	}
	SubSectionDebugEntries.Empty();
#endif
}

bool UDaySequenceModifierComponent::CanBeEnabled() const
{
	if (!bIsComponentEnabled)
	{
		return false;
	}
	
	if (Mode == EDaySequenceModifierMode::Volume)
	{
		AActor* Actor = TargetActor ? TargetActor.Get() : GetOwner();
		ENetMode NetMode = Actor->GetNetMode();
		return NetMode != NM_DedicatedServer;
	}

	return true;
}

void UDaySequenceModifierComponent::EnableComponent()
{
	if (bIsComponentEnabled)
	{
		return;
	}
	
	bIsComponentEnabled = true;
}

void UDaySequenceModifierComponent::DisableComponent()
{
	if (!bIsComponentEnabled && !bIsEnabled)
	{
		return;
	}
	
	bIsComponentEnabled = false;

	DisableModifier();
	RemoveSubSequenceTrack();
}

void UDaySequenceModifierComponent::EnableModifier()
{
	if (bIsEnabled || !CanBeEnabled())
	{
		return;
	}

	if (!bPreview && GetWorld()->WorldType == EWorldType::Editor)
	{
		return;
	}

	bIsEnabled = true;

	// Will call SetSubTrackMuteState for all living subsections, which checks enable state of modifier and their conditions
	InvalidateMuteStates();

	// In both collection and non collection case this array is populated, so if size is 0 we never initialized or removed subsections
	if (SubSections.Num() == 0)
	{
		ReinitializeSubSequence(nullptr);
	}

	SetInitialTimeOfDay();

	// Force an update if it's not playing so that the effects of this being enabled are seen
	if (TargetActor && !TargetActor->IsPlaying())
	{
		TargetActor->SetTimeOfDay(TargetActor->GetTimeOfDay());
	}

	OnPostEnableModifier.Broadcast();
}

void UDaySequenceModifierComponent::DisableModifier()
{
	if (!bIsEnabled)
	{
		return;
	}

	if (!bPreview && GetWorld()->WorldType == EWorldType::Editor)
	{
		return;
	}
	
	bIsEnabled = false;

	if (TargetActor && !TargetActor->HasAnyFlags(RF_BeginDestroyed))
	{
		// Will call SetSubTrackMuteState for all living subsections, which checks enable state of modifier and their conditions
		InvalidateMuteStates();

		TargetActor->UnregisterStaticTimeContributor(this);
		
		// Force an update if it's not playing so that the effects of this being disabled are seen
		if (!TargetActor->IsPlaying())
		{
			TargetActor->SetTimeOfDay(TargetActor->GetTimeOfDay());
		}
	}
}

void UDaySequenceModifierComponent::SetInitialTimeOfDay()
{
	using namespace UE::DaySequence;
	
	if (TargetActor)
	{
		const bool bHasAuthority = TargetActor->HasAuthority();
		const bool bRandomTimeOfDay = DayNightCycle == EDayNightCycleMode::RandomFixedTime || DayNightCycle == EDayNightCycleMode::RandomStartTime;
		const float RandomTime = FMath::FRand()*TargetActor->GetDayLength();
		const float Time = bRandomTimeOfDay ? RandomTime : DayNightCycleTime;

		switch (DayNightCycle)
		{
		case EDayNightCycleMode::FixedTime:
		case EDayNightCycleMode::RandomFixedTime:
		case EDayNightCycleMode::LocalFixedTime:
			{
				auto WantsStaticTime = [this]() -> bool
				{
					return IsValidChecked(this) && bIsEnabled && bIsComponentEnabled &&
						(DayNightCycle == EDayNightCycleMode::FixedTime ||
						DayNightCycle == EDayNightCycleMode::RandomFixedTime ||
						DayNightCycle == EDayNightCycleMode::LocalFixedTime);
				};

				auto GetStaticTime = [this, RandomTime, WantsStaticTime](FStaticTimeInfo& OutRequest) -> bool
				{
					if (WantsStaticTime())
					{
						OutRequest.BlendWeight = GetBlendWeight();
						OutRequest.StaticTime = DayNightCycle == EDayNightCycleMode::RandomFixedTime ? RandomTime : DayNightCycleTime;
						return true;
					}

					return false;
				};

				EStaticTimeChannel Channel = DayNightCycle == EDayNightCycleMode::LocalFixedTime ? EStaticTimeChannel::LocalFixedTime : EStaticTimeChannel::FixedTime;
				TargetActor->RegisterStaticTimeContributor({this, Bias, WantsStaticTime, GetStaticTime, Channel});
			}
			break;

		case EDayNightCycleMode::StartAtSpecifiedTime:
		case EDayNightCycleMode::RandomStartTime:
			if (!bHasAuthority && Mode != EDaySequenceModifierMode::Volume)
			{
				// Never set initial time of day from non-volume based modifiers if they don't have authority and aren't setting static time.
				// We'll just get the initial time of day from the server replication.
				return;
			}
			
			TargetActor->SetTimeOfDay(Time);
#if WITH_EDITOR
			TargetActor->ConditionalSetTimeOfDayPreview(Time);
#endif
			break;

#if WITH_EDITOR
		default:
			TargetActor->SetTimeOfDayPreview(TargetActor->GetTimeOfDayPreview());
			break;
#endif
		}
	}
}

void UDaySequenceModifierComponent::ReinitializeSubSequences()
{
	ReinitializeSubSequence(nullptr);
}

void UDaySequenceModifierComponent::ReinitializeSubSequence(ADaySequenceActor::FSubSectionPreserveMap* SectionsToPreserve)
{
	CSV_SCOPED_TIMING_STAT(DaySequence, ReinitializeSubSequence);
	
#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	bool bReinit = true;

	if (SectionsToPreserve)
	{
		// Mark all subsections we have recorded for keep in the root sequence
		// This is a fast path we take only if all of our subsections are in the root sequence
		for (TWeakObjectPtr<UMovieSceneSubSection> SubSection : SubSections)
		{
			if (const UMovieSceneSubSection* StrongSubSection = SubSection.Get())
			{
				if (bool* SectionToPreserveFlag = SectionsToPreserve->Find(StrongSubSection))
				{
					*SectionToPreserveFlag = true;
					bReinit = false;
				}
				else
				{
					// If we have a subsection that is not in the root sequence, break and reinit completely
					bReinit = true;
					break;
				}
			}
		}

		if (bReinit)
		{
			// Mark all sections associated with this modifier for delete before we do a full reinit
			for (TWeakObjectPtr<UMovieSceneSubSection> SubSection : SubSections)
			{
				if (const UMovieSceneSubSection* StrongSubSection = SubSection.Get())
				{
					if (bool* SectionToPreserveFlag = SectionsToPreserve->Find(StrongSubSection))
					{
						*SectionToPreserveFlag = false;
					}
				}
			}
		}
	}

	if (bReinit)
	{
#endif
		RemoveSubSequenceTrack();
		
		if (bUseCollection)
		{
			for (UDaySequenceCollectionAsset* Collection : DaySequenceCollections)
			{
				if (!Collection)
				{
					continue;
				}
				
				for (const FDaySequenceCollectionEntry& Entry : Collection->DaySequences)
				{
					InitializeDaySequence(Entry);
				}

				for (TInstancedStruct<FProceduralDaySequence>& ProceduralEntry : Collection->ProceduralDaySequences)
				{
					if (!ProceduralEntry.IsValid())
					{
						continue;
					}
			
					FProceduralDaySequence& ProceduralSequence = ProceduralEntry.GetMutable<FProceduralDaySequence>();
 
					if (UDaySequence* Sequence = ProceduralSequence.GetSequence(TargetActor))
					{
						FDaySequenceCollectionEntry TempEntry(Sequence);
						TempEntry.Conditions = ProceduralSequence.Conditions;
  
						InitializeDaySequence(TempEntry);
					}
				}
			}
		}
		else
		{
			InitializeDaySequence(UserDaySequence ? UserDaySequence : TransientSequence);
		}
#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	}
	else
	{
		// If we took the fast path, invalidate all mute states.
		InvalidateMuteStates();
	}
#endif
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	if (TargetActor)
	{
		if (!TargetActor->IsDebugCategoryRegistered(TargetActor->ShowDebug_SubSequenceCategory))
		{
			TargetActor->RegisterDebugCategory(TargetActor->ShowDebug_SubSequenceCategory,  TargetActor->OnShowDebugInfoDrawFunction);
		}
	
		for (TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry> Entry : SubSectionDebugEntries)
		{
			TargetActor->RegisterDebugEntry(Entry, TargetActor->ShowDebug_SubSequenceCategory);
		}
	}
#endif
	
	OnPostReinitializeSubSequences.Broadcast();
}

UMovieSceneSubSection* UDaySequenceModifierComponent::InitializeDaySequence(const FDaySequenceCollectionEntry& Entry)
{
	UDaySequence* RootSequence = TargetActor ? TargetActor->GetRootSequence() : nullptr;
    UMovieScene*  MovieScene   = RootSequence ? RootSequence->GetMovieScene() : nullptr;

    if (!MovieScene)
    {
    	return nullptr;
    }

	if (const UWorld* World = GetWorld(); World && World->WorldType != EWorldType::Editor && TargetActor->GetNetMode() == NM_DedicatedServer)
	{
		return nullptr;
	}
	
	auto CreateSubTrack = [this, MovieScene, bEnableBlending = Entry.bEnableHierarchicalBlending](UDaySequence* Sequence, int BiasOffset, bool bActivate)
	{
		UDaySequenceTrack*     RootTrack  = MovieScene->AddTrack<UDaySequenceTrack>();
		RootTrack->ClearFlags(RF_Transactional);
		RootTrack->SetFlags(RF_Transient);
        
		UMovieSceneSubSection* SubSection = CastChecked<UMovieSceneSubSection>(RootTrack->CreateNewSection());
		SubSection->ClearFlags(RF_Transactional);
		// SubSections of DaySequenceTrack will inherit flags from its parent track - RF_Transient in this case.
		SubSection->Parameters.HierarchicalBias = Bias + BiasOffset;
		SubSection->Parameters.Flags            = EMovieSceneSubSectionFlags::OverrideRestoreState
												| (bIgnoreBias ? EMovieSceneSubSectionFlags::IgnoreHierarchicalBias : EMovieSceneSubSectionFlags::None)
												| (bEnableBlending ? EMovieSceneSubSectionFlags::BlendHierarchicalBias : EMovieSceneSubSectionFlags::None);

		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		SubSection->SetSequence(Sequence);
		SubSection->SetRange(PlaybackRange);
		SubSection->SetIsActive(bActivate);

		if (DayNightCycle == EDayNightCycleMode::LocalFixedTime)
		{
			if (FMovieSceneTimeWarpVariant* Variant = SubSection->GetTimeWarp())
			{
				if (UMovieScene* SubMovieScene = Sequence ? Sequence->GetMovieScene() : nullptr)
				{
					const FFrameRate TickResolution = SubMovieScene->GetTickResolution();
					const FQualifiedFrameTime SubDuration = FQualifiedFrameTime(
						UE::MovieScene::DiscreteSize(SubMovieScene->GetPlaybackRange()),
						TickResolution);

					const float TargetFixedRatio = FMath::Fmod(DayNightCycleTime / TargetActor->GetDayLength(), 1.0f);
					const FTimecode TargetFixedTimecode = FTimecode(TargetFixedRatio * SubDuration.AsSeconds(), TickResolution, false, false);
					const FFrameNumber TargetFrame = TargetFixedTimecode.ToFrameNumber(TickResolution);
					Variant->Set(FMovieSceneTimeWarpFixedFrame{TargetFrame});
				}
			}
		}
		else
		{
			TargetActor->UpdateSubSectionTimeScale(SubSection);
		}

		RootTrack->AddSection(*SubSection);

		// In the Sequencer Editor, EaseIn pads the Sequence asset name by the EaseIn duration
		// (see SSequencerSection::OnPaint). Since we set the Easing duration to the full section
		// width to facilitate blending, the label is clipped. So we use EaseOut here instead and
		// ensure that the weight is inverted in Evaluate().
		SubSection->Easing.bManualEaseOut = true;
		SubSection->Easing.ManualEaseOutDuration = PlaybackRange.Size<FFrameNumber>().Value;
		EasingFunction->Initialize(UDaySequenceModifierEasingFunction::EEasingFunctionType::EaseOut);
		SubSection->Easing.EaseOut = EasingFunction;

#if WITH_EDITOR
		FString Label = GetOwner()->GetActorLabel();
#else
		FString Label = GetOwner()->GetName();
#endif
#if WITH_EDITORONLY_DATA
		RootTrack->DisplayName = FText::Format(LOCTEXT("ModifierTrackFormat", "Modifier ({0})"), FText::FromString(Label));
#endif
        
		SubSection->MarkAsChanged();
		SubSection->SetIsLocked(true);
		return SubSection;
	};
	
	constexpr bool bActivate = true;
	UMovieSceneSubSection* SubSection = CreateSubTrack(Entry.Sequence, Entry.BiasOffset, bActivate);
	
	if (!SubSections.Contains(SubSection))
	{
		SubSections.Add(SubSection);
	}

	if (Entry.Sequence)
	{
		if (const FGuid CameraModifierBindingGuid = Entry.Sequence->GetSpecializedBinding(EDaySequenceBindingReferenceSpecialization::CameraModifier); CameraModifierBindingGuid.IsValid())
		{
			TargetActor->RegisterBindingResolveFunction(SubSection->GetSequenceID(), CameraModifierBindingGuid, [this](TArray<UObject*, TInlineAllocator<1>>& InOutObjects) -> bool
			{
				if (TargetActor)
				{
					UObject* CameraModifier = TargetActor->GetCameraModifierManager()->GetCameraModifier(WeakBlendTarget.Get());
					InOutObjects.Add(CameraModifier);
				}
						
				return false; // don't allow default lookup
			});
		}
	}
	
	const TFunction<void(void)> SetSubTrackMuteStateConditional = [this, SubSection, Conditions = Entry.Conditions.Conditions]()
	{
		if (!IsValidChecked(this) || !IsValid(SubSection))
		{
			return;
		}

		SubSection->SetIsLocked(false);
		// Begin SubSection mutation:
		
		constexpr bool bInitialMuteState = false;
		const bool bActive = bIsEnabled && !TargetActor->EvaluateSequenceConditions(bInitialMuteState, Conditions);
		if (SubSection->IsActive() != bActive)
		{
			SubSection->MarkAsChanged();
			SubSection->SetIsActive(bActive);
		}

		SubSection->SetIsLocked(true);
	};

	const TFunction<void(void)> SetSubTrackMuteStateUnconditional = [this, SubSection]()
	{
		if (!IsValidChecked(this) || !IsValid(SubSection))
		{
			return;
		}

		SubSection->SetIsLocked(false);
		// Begin SubSection mutation:
		
		const bool bActive = bIsEnabled;
		if (SubSection->IsActive() != bActive)
		{
			SubSection->MarkAsChanged();
			SubSection->SetIsActive(bActive);
		}

		SubSection->SetIsLocked(true);
	};

	const TFunction<void(void)>& SetSubTrackMuteState = Entry.Conditions.Conditions.Num() == 0 ? SetSubTrackMuteStateUnconditional : SetSubTrackMuteStateConditional;
	
	// Initialize mute state and set up the condition callbacks to dynamically update mute state.
	SetSubTrackMuteState();
	OnInvalidateMuteStates.AddWeakLambda(SubSection, SetSubTrackMuteState);
	TargetActor->BindToConditionCallbacks(this, Entry.Conditions.Conditions, [this]() { InvalidateMuteStates(); });
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG	
	// This gets captured by a lambda below so should continue living
	TSharedPtr<TMap<FString, FString>> DebugData = MakeShared<TMap<FString, FString>>();
	SubSectionDebugEntries.Emplace(MakeShared<UE::DaySequence::FDaySequenceDebugEntry>(
	[this](){ return ShouldShowDebugInfo(); },
	[this, DebugData, SubSection]()
	{
		if (IsValid(SubSection))
		{
			(*DebugData).FindOrAdd("Name") = GetPathName(GetOwner() ? GetOwner()->GetOuter() : nullptr);
			(*DebugData).FindOrAdd("Authority") = GetOwner()->HasAuthority() ? "True" : "False";
			(*DebugData).FindOrAdd("Sequence Name") = SubSection->GetSequence() ? SubSection->GetSequence()->GetFName().ToString() : "None";
			(*DebugData).FindOrAdd("Mute State") = SubSection->IsActive() ? "Active" : "Muted";
			(*DebugData).FindOrAdd("Hierarchical Bias") = FString::Printf(TEXT("%d"), SubSection->Parameters.HierarchicalBias);
		}
	
		return DebugData;
	},
	[SubSection]()
	{
		return IsValid(SubSection) ? SubSection->Parameters.HierarchicalBias : 0;
	}));
#endif

	return SubSection;
}

void UDaySequenceModifierComponent::SetUserDaySequence(UDaySequence* InDaySequence)
{
	if (InDaySequence && InDaySequence->HasAnyFlags(RF_Transient))
	{
		FFrame::KismetExecutionMessage(TEXT("SetUserDaySequence called with a transient sequence, use SetTransientSequence instead!"), ELogVerbosity::Error);
		return;
	}

	// Prevents unnecessary & expensive subsequence reinitialization
	if (InDaySequence == UserDaySequence)
	{
		return;
	}

	UserDaySequence = InDaySequence;
	ReinitializeSubSequence(nullptr);
}

UDaySequence* UDaySequenceModifierComponent::GetUserDaySequence() const
{
	return UserDaySequence;
}

void UDaySequenceModifierComponent::SetTransientSequence(UDaySequence* InDaySequence)
{
	if (InDaySequence && !InDaySequence->HasAnyFlags(RF_Transient))
	{
		FFrame::KismetExecutionMessage(TEXT("SetTransientSequence called with a non-transient sequence, use SetUserDaySequence instead!"), ELogVerbosity::Error);
		return;
	}

	// Prevents unnecessary & expensive subsequence reinitialization
	if (InDaySequence == TransientSequence)
	{
		return;
	}
	
	TransientSequence = InDaySequence;
	ReinitializeSubSequence(nullptr);
}
	
UDaySequence* UDaySequenceModifierComponent::GetTransientSequence() const
{
	return TransientSequence;
}

void UDaySequenceModifierComponent::SetDayNightCycle(EDayNightCycleMode NewMode)
{
	DayNightCycle = NewMode;

	if (bIsComponentEnabled)
	{
		DisableComponent();
		EnableComponent();
	}
}

EDayNightCycleMode UDaySequenceModifierComponent::GetDayNightCycle() const
{
	return DayNightCycle;
}

void UDaySequenceModifierComponent::SetBias(int32 NewBias)
{
	Bias = NewBias;
}
int32 UDaySequenceModifierComponent::GetBias() const
{
	return Bias;
}

void UDaySequenceModifierComponent::SetDayNightCycleTime(float Time)
{
	DayNightCycleTime = Time;
}

float UDaySequenceModifierComponent::GetDayNightCycleTime() const
{
	return DayNightCycleTime;
}

void UDaySequenceModifierComponent::SetMode(EDaySequenceModifierMode NewMode)
{
	Mode = NewMode;
}

EDaySequenceModifierMode UDaySequenceModifierComponent::GetMode() const
{
	return Mode;
}

void UDaySequenceModifierComponent::SetBlendPolicy(EDaySequenceModifierUserBlendPolicy NewPolicy)
{
	BlendPolicy = NewPolicy;
}

EDaySequenceModifierUserBlendPolicy UDaySequenceModifierComponent::GetBlendPolicy() const
{
	return BlendPolicy;
}

void UDaySequenceModifierComponent::SetBlendTarget(APlayerController* InActor)
{	
	WeakBlendTarget = InActor;
}

void UDaySequenceModifierComponent::SetBlendTime(float TimeInSeconds)
{
	BlendTime = TimeInSeconds;
}

float UDaySequenceModifierComponent::GetBlendTime() const
{
	return BlendTime;
}

void UDaySequenceModifierComponent::SetUserBlendWeight(float Weight)
{
	UserBlendWeight = FMath::Clamp(Weight, 0.f, 1.f);
}

float UDaySequenceModifierComponent::GetUserBlendWeight() const
{
	return UserBlendWeight;
}

bool UDaySequenceModifierComponent::GetBlendPosition(FVector& InPosition) const
{
	CSV_SCOPED_TIMING_STAT(DaySequence, GetBlendPosition);
	
#if WITH_EDITOR
	if (const UWorld* World = GetWorld(); World && (!World->IsGameWorld() || UE::DaySequence::bIsSimulating))
	{
		InPosition = UE::DaySequence::GVolumePreviewLocation;
		return true;
	}
	else
#endif
	if (const APlayerController* BlendTarget = WeakBlendTarget.Get();
		BlendTarget && BlendTarget->PlayerCameraManager &&
		(!UE::DaySequence::CVarModifierBlendPositionValidateTimestamp.GetValueOnAnyThread() || BlendTarget->PlayerCameraManager->GetCameraCacheTime() != 0.f))
	{
		CSV_SCOPED_TIMING_STAT(DaySequence, GetPlayerViewPoint);
		InPosition = BlendTarget->PlayerCameraManager->GetCameraLocation();
		return true;
	}

	return false;
}

TArray<UShapeComponent*> UDaySequenceModifierComponent::GetVolumeShapeComponents() const
{
	TArray<UShapeComponent*> ResolvedVolumeShapeComponents;
	ResolvedVolumeShapeComponents.Reserve(VolumeShapeComponents.Num());
	
	if (bCachedExternalShapesInvalid)
	{
		UpdateCachedExternalShapes();
	}
	
#if WITH_EDITOR
	// We don't expect changes to VolumeShapeComponents or CachedExternalShapes during play.
	// This will ensure that CachedExternalShapes remains updated to reflect editor workflows
	// that might invalidate entries, such as deleting a shape component on an external actor.
	bool bRecache = false;
#endif

	// This loop serves two purposes:
	// 1) Go ahead and move from weak object ptr to raw pointer so the caller doesn't have to.
	// 2) Determine if CachedExternalShapes is invalid so we can recache (occurs when reference shape component is deleted).
	for (TWeakObjectPtr<UShapeComponent> Shape : CachedExternalShapes)
	{
		if (UShapeComponent* ValidShape = Shape.Get())
		{
			ResolvedVolumeShapeComponents.Add(ValidShape);
		}
#if WITH_EDITOR
		else
		{
			// Break out out here as we will update the cached shapes and reconstruct ResolvedVolumeShapeComponents below.
			bRecache = true;
			break;
		}
#endif
	}

#if WITH_EDITOR
	// We do this here so that we don't modify CachedExternalShapes while iterating over it.
	// The idea is if we recache immediately before the recursive call, we should not be able to recursively hit this branch.
	if (bRecache)
	{
		checkNoRecursion();
		bCachedExternalShapesInvalid = true;
		UpdateCachedExternalShapes();
		return GetVolumeShapeComponents();
	}
#endif
	
	return ResolvedVolumeShapeComponents;
}

float UDaySequenceModifierComponent::GetBlendWeight() const
{
	switch (BlendPolicy)
	{
	default:
	case EDaySequenceModifierUserBlendPolicy::Ignored:
		return InternalBlendWeight;
		
	case EDaySequenceModifierUserBlendPolicy::Minimum:
		return FMath::Min(InternalBlendWeight, UserBlendWeight);
		
	case EDaySequenceModifierUserBlendPolicy::Maximum:
		return FMath::Max(InternalBlendWeight, UserBlendWeight);
		
	case EDaySequenceModifierUserBlendPolicy::Override:
		return UserBlendWeight;
	}
}

float UDaySequenceModifierComponent::UpdateInternalBlendWeight()
{
	CSV_SCOPED_TIMING_STAT(DaySequence, UpdateInternalBlendWeight);

	const float PreviousBlendWeight = InternalBlendWeight;
	
	switch (Mode)
	{
	default:
	case EDaySequenceModifierMode::Global:
		InternalBlendWeight = 1.f;
		break;

	case EDaySequenceModifierMode::Time:
		// Intentional fallthrough in order to determine if we are currently inside or outside the volume
	case EDaySequenceModifierMode::Volume:
		if (FVector BlendPosition; GetBlendPosition(BlendPosition))
		{
			InternalBlendWeight = 0.f;

			auto GetBlendWeightForShape = [this, BlendPosition](const UShapeComponent* Shape)
			{
				const float Distance = UE::DaySequence::ComputeSignedDistance(Shape, BlendPosition);
				return Distance < 0.f ? FMath::Clamp(-Distance / BlendAmount, 0.f, 1.f) : 0.f;
			};
			
			for (const UShapeComponent* Shape : GetVolumeShapeComponents())
			{
				InternalBlendWeight = FMath::Max(InternalBlendWeight, GetBlendWeightForShape(Shape));
			}
		}
		else
		{
			InternalBlendWeight = 0.f;
		}
		break;
	}
	
	if (BlendTime > 0.0f)
	{
		if (const UWorld* World = GetWorld())
		{
			const float CurrentTime = World->GetTimeSeconds();
			float DeltaWeightRange;
			float DeltaTime;
			if (Mode == EDaySequenceModifierMode::Time)
			{
				TimedBlendingTargetWeight = InternalBlendWeight > 0.f ? 1.f : 0.f;
				TimedBlendingStartWeight = PreviousBlendWeight;
				DeltaTime = CurrentTime - TimedBlendingLastUpdated;
				DeltaWeightRange = InternalBlendWeight > 0.f ? 1.f : -1.f;
			}
			else // Global || Volume
			{
				if (FMath::Abs(InternalBlendWeight - TimedBlendingTargetWeight) > 0.f)
				{
					TimedBlendingTargetWeight = InternalBlendWeight;
					TimedBlendingStartWeight = PreviousBlendWeight;
					TimedBlendingTargetWeightLastUpdated = TimedBlendingLastUpdated == 0.f ? CurrentTime : TimedBlendingLastUpdated;
				}
				DeltaTime = CurrentTime - TimedBlendingTargetWeightLastUpdated;
				DeltaWeightRange = TimedBlendingTargetWeight - TimedBlendingStartWeight;
			}

			TimedBlendingLastUpdated = CurrentTime;
			
			const float DeltaTimeRatio = FMath::Clamp(DeltaTime / FMath::Max(BlendTime, UE_SMALL_NUMBER), 0.f, 1.f);
			const float DeltaWeight = DeltaWeightRange * DeltaTimeRatio;

			const float MinWeight = DeltaWeightRange >= 0.f ? TimedBlendingStartWeight : TimedBlendingTargetWeight;
			const float MaxWeight = DeltaWeightRange >= 0.f ? TimedBlendingTargetWeight : TimedBlendingStartWeight;
			InternalBlendWeight = FMath::Clamp(TimedBlendingStartWeight + DeltaWeight, MinWeight, MaxWeight);
			
			UE_LOGF(LogDaySequence, VeryVerbose, "[BlendWeight]: Start = %f, Target = %f, DeltaTime = %f, Weight = %f", TimedBlendingStartWeight, TimedBlendingTargetWeight, DeltaTime, InternalBlendWeight);
		}
		else
		{
			InternalBlendWeight = PreviousBlendWeight;
		}
	}
	
	if (UE::DaySequence::CVarModifierDisableWhenInvisible.GetValueOnAnyThread())
	{
		if (!IsVisible())
		{
			InternalBlendWeight = 0.f;
		}
	}

	return InternalBlendWeight;
}

void UDaySequenceModifierComponent::EmptyVolumeShapeComponents()
{
	VolumeShapeComponents.Empty();
	bCachedExternalShapesInvalid = true;
}

void UDaySequenceModifierComponent::AddVolumeShapeComponent(const FComponentReference& InShapeReference)
{
	VolumeShapeComponents.AddUnique(InShapeReference);
	bCachedExternalShapesInvalid = true;
}

FOnPostEnableModifier& UDaySequenceModifierComponent::GetOnPostEnableModifier()
{
	return OnPostEnableModifier;
}

void UDaySequenceModifierComponent::InvalidateMuteStates() const
{
	OnInvalidateMuteStates.Broadcast();
}

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
void UDaySequenceModifierComponent::OnDebugLevelChanged(int32 InDebugLevel)
{
	DebugLevel = InDebugLevel;
}

bool UDaySequenceModifierComponent::ShouldShowDebugInfo() const
{
	if (!TargetActor || TargetActor->GetNetMode() == NM_DedicatedServer)
	{
		return false;
	}
	
	switch (DebugLevel)
	{
	case 0: return false;
	case 1: return bIsEnabled;
	case 2: return bIsComponentEnabled;
	case 3: return true;
	default: return false;
	}
}
#endif

void UDaySequenceModifierComponent::UpdateCachedExternalShapes() const
{
	check (bCachedExternalShapesInvalid)
	
	CachedExternalShapes.Reset();
	
	for (const FComponentReference& ComponentRef : VolumeShapeComponents)
	{
		if (ComponentRef.PathToComponent.Len() != 0 || ComponentRef.ComponentProperty != NAME_None || !ComponentRef.OverrideComponent.IsExplicitlyNull())
		{
			if (UShapeComponent* ResolvedShape = Cast<UShapeComponent>(ComponentRef.GetComponent(GetOwner())); IsValid(ResolvedShape))
			{
				CachedExternalShapes.Add(ResolvedShape);
			}
		}
	}

	bCachedExternalShapesInvalid = false;
}

#undef LOCTEXT_NAMESPACE
