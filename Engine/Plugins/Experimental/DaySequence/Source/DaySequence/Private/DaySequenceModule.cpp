// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceModule.h"
#include "DaySequenceActor.h"
#include "DaySequenceActorSpawner.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneTracksComponentTypes.h"

#include <limits>

#define LOCTEXT_NAMESPACE "FDaySequenceModule"

DEFINE_LOG_CATEGORY(LogDaySequence);
CSV_DEFINE_CATEGORY(DaySequence, false);

void FDaySequenceModule::StartupModule()
{
	using namespace UE::DaySequence;

	OnCreateMovieSceneObjectSpawnerDelegateHandle = RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner::CreateStatic(&FDaySequenceActorSpawner::CreateObjectSpawner));
}

void FDaySequenceModule::ShutdownModule()
{
	UnregisterObjectSpawner(OnCreateMovieSceneObjectSpawnerDelegateHandle);
}

FDelegateHandle FDaySequenceModule::RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner InOnCreateMovieSceneObjectSpawner)
{
	OnCreateMovieSceneObjectSpawnerDelegates.Add(InOnCreateMovieSceneObjectSpawner);
	return OnCreateMovieSceneObjectSpawnerDelegates.Last().GetHandle();
}

void FDaySequenceModule::UnregisterObjectSpawner(FDelegateHandle InHandle)
{
	OnCreateMovieSceneObjectSpawnerDelegates.RemoveAll([=](const FOnCreateMovieSceneObjectSpawner& Delegate) { return Delegate.GetHandle() == InHandle; });
}

void FDaySequenceModule::GenerateObjectSpawners(TArray<TSharedRef<IMovieSceneObjectSpawner>>& OutSpawners) const
{
	for (const FOnCreateMovieSceneObjectSpawner& SpawnerFactory : OnCreateMovieSceneObjectSpawnerDelegates)
	{
		check(SpawnerFactory.IsBound());
		OutSpawners.Add(SpawnerFactory.Execute());
	}

	// Now sort the spawners. Editor spawners should come first so they override runtime versions of the same supported type in-editor.
	// @TODO: we could also sort by most-derived type here to allow for type specific behaviors
	OutSpawners.Sort([](TSharedRef<IMovieSceneObjectSpawner> LHS, TSharedRef<IMovieSceneObjectSpawner> RHS)
	{
		return LHS->IsEditor() > RHS->IsEditor();
	});
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDaySequenceModule, DaySequence)