// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDaySequenceModule.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDaySequence, Log, All);
CSV_DECLARE_CATEGORY_EXTERN(DaySequence);
DECLARE_STATS_GROUP(TEXT("Day Sequence Evaluation"), STATGROUP_DaySequenceEval, STATCAT_Advanced);

class FDaySequenceModule : public IDaySequenceModule
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual FDelegateHandle RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner InOnCreateMovieSceneObjectSpawner) override;
	virtual void GenerateObjectSpawners(TArray<TSharedRef<IMovieSceneObjectSpawner>>& OutSpawners) const override;
	virtual void UnregisterObjectSpawner(FDelegateHandle InHandle) override;

public:
	/** List of object spawner delegates used to extend the spawn register */
	TArray<FOnCreateMovieSceneObjectSpawner> OnCreateMovieSceneObjectSpawnerDelegates;

	/** Internal delegate handle used for spawning actors */
	FDelegateHandle OnCreateMovieSceneObjectSpawnerDelegateHandle;
};
