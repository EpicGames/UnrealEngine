// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequenceActor.h"
#include "DaySequenceModule.h"
#include "IDaySequencePlayer.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneLatentActionManager.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieScenePlayback.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSequencePlayer.h"

#include "DaySequencePlayer.generated.h"

#define UE_API DAYSEQUENCE_API

class UDaySequencePlayer;
class UDaySequence;
struct FMovieSceneSequencePlaybackSettings;

class AActor;
class ADaySequenceActor;

namespace UE::MovieScene
{
class FSequenceWeights;
}

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDaySequencePlayerEvent);
DECLARE_DELEGATE(FOnDaySequencePlayerNativeEvent);

USTRUCT()
struct FDaySequencePlaybackParams
{
	GENERATED_BODY()

	FDaySequencePlaybackParams()
		: UpdateMethod(EUpdatePositionMethod::Play)
		, bHasJumped(false)
	{}

	FDaySequencePlaybackParams(FFrameTime InFrame, EUpdatePositionMethod InUpdateMethod)
		: Frame(InFrame)
		, UpdateMethod(InUpdateMethod)
		, bHasJumped(false)
	{}

	// Get the playback position using the player's tick resolution and display rate	
	FFrameTime GetPlaybackPosition (UDaySequencePlayer* Player) const;

	FFrameTime Frame;

	EUpdatePositionMethod UpdateMethod;

	bool bHasJumped;
};

/**
 * UDaySequencePlayer is used to actually "play" a Day sequence asset at runtime.
 *
 * This class keeps track of playback state and provides functions for manipulating
 * a DaySequence while its playing.
 */
UCLASS(MinimalAPI)
class UDaySequencePlayer
	: public UObject
	, public IMovieScenePlayer
	, public IDaySequencePlayer
{
public:
	GENERATED_BODY()

	friend class UE::DaySequence::FOverrideUpdateIntervalHandle;

	UE_API UDaySequencePlayer(const FObjectInitializer&);
	UE_API virtual ~UDaySequencePlayer() override;

	/** Obeserver interface used for controlling whether this sequence can be played. */
	UPROPERTY(replicated)
	TScriptInterface<IMovieSceneSequencePlayerObserver> Observer;

	/** Start playback forwards from the current time cursor position, using the current play rate. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void Play();

	/**
	 * Start playback from the current time cursor position, looping the specified number of times.
	 * @param NumLoops - The number of loops to play. -1 indicates infinite looping.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void PlayLooping(int32 NumLoops = -1);
	
	/** Pause playback. */
	UE_API virtual void Pause() override;
	
	/** Scrub playback. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void Scrub();

	/** Stop playback and move the cursor to the end (or start, for reversed playback) of the sequence. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void Stop();

	/** Stop playback without moving the cursor. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void StopAtCurrentTime();

	/** Go to end and stop. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", meta = (ToolTip = "Go to end of the sequence and stop. Adheres to 'When Finished' section rules."))
	UE_API void GoToEndAndStop();

	UE_API virtual TSharedPtr<UE::DaySequence::FOverrideUpdateIntervalHandle> GetOverrideUpdateIntervalHandle() override;
	
public:

	UE_API virtual bool HasEverEvaluated() const override;

	/**
	 * Get the current playback position
	 * @return The current playback position
	 */
	UE_API virtual FQualifiedFrameTime GetCurrentTime() const override;

	/**
	 * Get the total duration of the sequence
	 */
	UE_API virtual FQualifiedFrameTime GetDuration() const override;

	/**
	 * Returns the number of times this sequence has been looped
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API virtual int32 GetCurrentNumLoops() const {return CurrentNumLoops;} ;
	
	/**
	 * Get this sequence's duration in frames
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API int32 GetFrameDuration() const;

	/**
	 * Get this sequence's display rate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FFrameRate GetFrameRate() const { return PlayPosition.GetInputRate(); }

	/**
	 * Set the frame-rate that this player should play with, making all frame numbers in the specified time-space
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void SetFrameRate(FFrameRate FrameRate);

	/**
	 * Get the offset within the level sequence to start playing
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FQualifiedFrameTime GetStartTime() const { return FQualifiedFrameTime(StartTime, PlayPosition.GetInputRate()); }

	/**
	 * Get the offset within the level sequence to finish playing
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FQualifiedFrameTime GetEndTime() const { return FQualifiedFrameTime(StartTime + DurationFrames, PlayPosition.GetInputRate()); }

	/**
	 * Set a manual weight to be multiplied with all blendable elements within this sequence
	 * @note: It is recommended that a weight between 0 and 1 is supplied, though this is not enforced
	 * @note: It is recommended that either FMovieSceneSequencePlaybackSettings::DynamicWeighting should be true for this player or the asset it's playing back should be set to enable dynamic weight to avoid undesirable behavior
	 *
	 * @param InWeight    The weight to suuply to all elements in this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void SetWeight(double InWeight);

	/**
	 * Removes a previously assigned weight
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void RemoveWeight();

	/**
	 * Set a manual weight to be multiplied with all blendable elements within the specified sequence
	 * @note: It is recommended that a weight between 0 and 1 is supplied, though this is not enforced
	 * @note: It is recommended that either FMovieSceneSequencePlaybackSettings::DynamicWeighting should be true for this player or the asset it's playing back should be set to enable dynamic weight to avoid undesirable behavior
	 *
	 * @param InWeight    The weight to suuply to all elements in this sequence
	 */
	UE_API void SetWeight(double InWeight, FMovieSceneSequenceID SequenceID);

	/**
	 * Removes a previously assigned weight
	 */
	UE_API void RemoveWeight(FMovieSceneSequenceID SequenceID);

public:

	/**
	 * Set the valid play range for this sequence, determined by a starting frame number (in this sequence player's plaback frame), and a number of frames duration
	 *
	 * @param StartFrame      The frame number to start playing back the sequence
	 * @param Duration        The number of frames to play
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", DisplayName="Set Play Range (Frames)")
	UE_API void SetFrameRange( int32 StartFrame, int32 Duration, float SubFrames = 0.f );

public:

	/**
	 * Set the current time of the player by evaluating from the current time to the specified time, as if the sequence is playing. 
	 * Triggers events that lie within the evaluated range. Does not alter the persistent playback status of the player (IsPlaying).
	 *
	 * @param PlaybackParams The position settings (ie. the position to set playback to)
	 */
	UE_API void SetPlaybackPosition (FDaySequencePlaybackParams PlaybackParams);

	/** Set the state of the completion mode override. Note, setting the state to force restore state will only take effect if the sequence hasn't started playing */
	UFUNCTION(BlueprintCallable, Category = "Game|Cinematic")
	UE_API void SetCompletionModeOverride(EMovieSceneCompletionModeOverride CompletionModeOverride);

	/** Get the state of the completion mode override */
	UFUNCTION(BlueprintCallable, Category = "Game|Cinematic")
	UE_API EMovieSceneCompletionModeOverride GetCompletionModeOverride() const;

public:

	/** Check whether the sequence is actively playing. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API bool IsPlaying() const;

	/** Check whether the sequence is paused. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API bool IsPaused() const;

	/** Get the playback rate of this player. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API float GetPlayRate() const;

	/**
	 * Set the playback rate of this player. Negative values will play the animation in reverse.
	 * @param PlayRate - The new rate of playback for the animation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void SetPlayRate(float PlayRate);

	/** Set whether to disable camera cuts */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	void SetDisableCameraCuts(bool bInDisableCameraCuts) { PlaybackSettings.bDisableCameraCuts = bInDisableCameraCuts; }

	/** Set whether to disable camera cuts */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	bool GetDisableCameraCuts() { return PlaybackSettings.bDisableCameraCuts; }

	/** An event that is broadcast each time this level sequence player is updated */
	DECLARE_EVENT_ThreeParams( UDaySequencePlayer, FOnDaySequencePlayerUpdated, const UDaySequencePlayer&, FFrameTime /*current time*/, FFrameTime /*previous time*/ );
	FOnDaySequencePlayerUpdated& OnSequenceUpdated() const { return OnDaySequencePlayerUpdate; }

	/** Event triggered when the level sequence player is played */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnDaySequencePlayerEvent OnPlay;

	/** Event triggered when the level sequence player is played in reverse */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnDaySequencePlayerEvent OnPlayReverse;

	/** Event triggered when the level sequence player is stopped */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnDaySequencePlayerEvent OnStop;

	/** Event triggered when the level sequence player is paused */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnDaySequencePlayerEvent OnPause;

	/** Event triggered when the level sequence player finishes naturally (without explicitly calling stop) */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnDaySequencePlayerEvent OnFinished;

	/** Native event triggered when the level sequence player finishes naturally (without explicitly calling stop) */
	FOnDaySequencePlayerNativeEvent OnNativeFinished;

public:

	/** Retrieve all objects currently bound to the specified binding identifier */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding);

	/** Get the object bindings for the requested object */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API TArray<FMovieSceneObjectBindingID> GetObjectBindings(UObject* InObject);

	/* Invalidates the given binding, forcing it to be refetched. This may be useful for some custom bindings that wish their resolution code to be called again.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void RequestInvalidateBinding(FMovieSceneObjectBindingID ObjectBinding);

public:

	/** Assign this player's playback settings */
	UE_API void SetPlaybackSettings(const FMovieSceneSequencePlaybackSettings& InSettings);

	/** Initialize this player using its existing playback settings */
	UE_API void Initialize(UMovieSceneSequence* InSequence);

	/** Initialize this player with a sequence and some settings */
	UE_API void Initialize(UMovieSceneSequence* InSequence, const FMovieSceneSequencePlaybackSettings& InSettings);

	/** Update the sequence for the current time, if playing */
	UE_API void Update(const float DeltaSeconds);

	/** Update the sequence for the current time, if playing, asynchronously */
	UE_API void UpdateAsync(const float DeltaSeconds);

	/** Perform any tear-down work when this player is no longer (and will never) be needed */
	UE_API void TearDown();

	/** Returns whether this player is valid, i.e. it has been initialized and not torn down yet */
	UE_API bool IsValid() const;

public:

	/**
	 * Access the sequence this player is playing
	 * @return the sequence currently assigned to this player
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UMovieSceneSequence* GetSequence() const { return Sequence; }

	/**
	 * Get the name of the sequence this player is playing
	 * @param bAddClientInfo  If true, add client index if running as a client
	 * @return the name of the sequence, or None if no sequence is set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API FString GetSequenceName(bool bAddClientInfo = false) const;

	/**
	 * Assign a playback client interface for this sequence player, defining instance data and binding overrides
	 */
	UE_API void SetPlaybackClient(TScriptInterface<IMovieScenePlaybackClient> InPlaybackClient);

	/**
	 * Retrieve the currently assigned time controller
	 */
	UE_API TSharedPtr<FMovieSceneTimeController> GetTimeController() const;

	/**
	 * Assign a time controller for this sequence player allowing custom time management implementations.
	 * Will reset the supplied time controller to the current time.
	 */
	UE_API void SetTimeController(TSharedPtr<FMovieSceneTimeController> InTimeController);
	

	/**
	 * Assign a time controller for this sequence player allowing custom time management implementations.
	 * Will not reset the supplied time controller in any way, so the sequence will receive its time directly from the controller.
	 */
	UE_API void SetTimeControllerDirectly(TSharedPtr<FMovieSceneTimeController> InTimeController);

	/**
	 * Sets whether to listen or ignore playback replication events.
	 * @param bState If true, ignores playback replication.
	 */
	UE_API virtual void SetIgnorePlaybackReplication(bool bState) override;

protected:

	UE_API void PlayInternal();
	UE_API void StopInternal(FFrameTime TimeToResetTo);
	UE_API void FinishPlaybackInternal(FFrameTime TimeToFinishAt);

	struct FMovieSceneUpdateArgs
	{
		bool bHasJumped = false;
		bool bIsAsync = false;
	};

	UE_API void UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, bool bHasJumped = false);
	UE_API virtual void UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, const FMovieSceneUpdateArgs& Args);

	UE_API void UpdateTimeCursorPosition(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride = false);
	UE_API bool ShouldStopOrLoop(FFrameTime NewPosition) const;

	UE_API UWorld* GetPlaybackWorld() const;

	UE_API FFrameTime GetLastValidTime() const;

	UE_API FFrameRate GetDisplayRate() const;

	UE_API bool NeedsQueueLatentAction() const;
	UE_API void QueueLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate);
	UE_API void RunLatentActions();

public:
	//~ IMovieScenePlayer interface
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return RootTemplateInstance; }

protected:
	//~ IMovieScenePlayer interface
	UE_API virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;
	UE_API virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;
	UE_API virtual FMovieSceneSpawnRegister& GetSpawnRegister() override;
	virtual UObject* AsUObject() override { return this; }

	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}

	UE_API virtual void ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	virtual IMovieScenePlaybackClient* GetPlaybackClient() override { return PlaybackClient ? &*PlaybackClient : nullptr; }
	UE_API virtual bool HasDynamicWeighting() const override;
	UE_API virtual void PreEvaluation(const FMovieSceneContext& Context) override;
	UE_API virtual void PostEvaluation(const FMovieSceneContext& Context) override;

	virtual TScriptInterface<IMovieSceneSequencePlayerObserver> GetObserver() override { return Observer; }

	/*~ Begin UObject interface */
	virtual bool IsSupportedForNetworking() const { return true; }
	UE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	UE_API virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;
	UE_API virtual void PostNetReceive() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
	/*~ End UObject interface */
	
	bool CanPlay() const { return WeakOwner.IsValid(); }
	
private:

	UE_API void UpdateTimeCursorPosition_Internal(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride);

	UE_API void RunPreEvaluationCallbacks();
	UE_API void RunPostEvaluationCallbacks();

	UE_API void IncrementServerSerialNumber();
	UE_API void AdvanceClientSerialNumberTo(int32 NewSerialNumber);
	
private:

	/**
	 * Called on the server whenever an explicit change in time has occurred through one of the (Play|Jump|Scrub)To methods
	 */
	UFUNCTION(netmulticast, reliable)
	UE_API void RPC_ExplicitServerUpdateEvent(EUpdatePositionMethod Method, FFrameTime RelevantTime, int32 NewSerialNumber);

	/**
	 * Called on the server when Stop() is called in order to differentiate Stops from Pauses.
	 */
	UFUNCTION(netmulticast, reliable)
	UE_API void RPC_OnStopEvent(FFrameTime StoppedTime, int32 NewSerialNumber);

	/**
	 * Called on the server when playback has reached the end. Could lead to stopping or pausing.
	 */
	UFUNCTION(netmulticast, reliable)
	UE_API void RPC_OnFinishPlaybackEvent(FFrameTime StoppedTime, int32 NewSerialNumber);

	/**
	 * Called on the server when PlaybackSettings.PlayRate changes so that clients can discard server time samples.
	 */
	UFUNCTION(netmulticast, reliable)
	UE_API void RPC_OnPlayRateChanged();
	
	/**
	 * Check whether this sequence player is an authority, as determined by its outer Actor
	 */
	UE_API bool HasAuthority() const;

	/**
	 * Update the replicated properties required for synchronizing to clients of this sequence player
	 */
	UE_API void UpdateNetworkSyncProperties();

	/**
	 * Analyse the set of samples we have estimating the server time if we have confidence over the data.
	 * Should only be called once per frame.
	 * @return An estimation of the server time, or the current local time if we cannot make a strong estimate
	 */
	UE_API FFrameTime UpdateServerTimeSamples();

	/**
	 * Check and correct network synchronization for the clients of this sequence player.
	 */
	UE_API void UpdateNetworkSync();

	/**
	 * Compute the latency for the client connection.
	 */
	UE_API float GetPing() const;

protected:

	/** Movie player status. */
	UPROPERTY()
	TEnumAsByte<EMovieScenePlayerStatus::Type> Status;

	/** Set to true to invoke OnStartedPlaying on first update tick for started playing */
	uint32 bPendingOnStartedPlaying : 1;

	/** Set to true when the player is currently in the main level update */
	uint32 bIsAsyncUpdate : 1;

	/** Flag that allows the player to tick its time controller without actually evaluating the sequence */
	uint32 bSkipNextUpdate : 1;

	/** Flag that notifies the player to check network synchronization on next update */
	uint32 bUpdateNetSync : 1;

	/** Flag that indicates whether to warn on zero duration playback */
	uint32 bWarnZeroDuration : 1;

	/** The sequence to play back */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneSequence> Sequence;

	/** Time (in playback frames) at which to start playing the sequence (defaults to the lower bound of the sequence's play range) */
	UPROPERTY(replicated)
	FFrameNumber StartTime;

	/** Time (in playback frames) at which to stop playing the sequence (defaults to the upper bound of the sequence's play range) */
	UPROPERTY(replicated)
	int32 DurationFrames;

	UPROPERTY(replicated)
	float DurationSubFrames;

	/** The number of times we have looped in the current playback */
	UPROPERTY(transient)
	int32 CurrentNumLoops;

	/**
	 * The serial number for the current update lifespan
	 * It is incremented every time we pass a "gate" such as an RPC call that stops/finishes the sequence.
	 */
	UPROPERTY(transient)
	int32 SerialNumber;

	/** Specific playback settings for the animation. */
	UPROPERTY(replicated)
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	/** The root template instance we're evaluating */
	UPROPERTY(transient)
	FMovieSceneRootEvaluationTemplateInstance RootTemplateInstance;

	/** Play position helper */
	FMovieScenePlaybackPosition PlayPosition;

	/** Spawn register */
	TSharedPtr<FMovieSceneSpawnRegister> SpawnRegister;

	/** Sequence Weights */
	TUniquePtr<UE::MovieScene::FSequenceWeights> SequenceWeights;

	struct FServerTimeSample
	{
		/** The actual server sequence time in seconds, with client ping at the time of the sample baked in */
		double ServerTime;
		/** Wall-clock time that the sample was receieved */
		double ReceivedTime;
	};
	/**
	 * Array of server sequence times in seconds, with ping compensation baked in.
	 * Samples are sorted chronologically with the oldest samples first
	 */
	TArray<FServerTimeSample> ServerTimeSamples;

	/*
	* On UpdateServerTimeSamples, the last recorded time dilation. Used to update the server time samples each update to ensure we can smooth server time even on changing time dilation.
	*/
	float LastEffectiveTimeDilation = 1.0f;

	/** Replicated playback status and current time that are replicated to clients */
	UPROPERTY(replicated)
	FMovieSceneSequenceReplProperties NetSyncProps;

	/** External client pointer in charge of playing back this sequence */
	UPROPERTY(Transient)
	TScriptInterface<IMovieScenePlaybackClient> PlaybackClient;

	/** Local latent action manager for when we're running a blocking sequence */
	FMovieSceneLatentActionManager LatentActionManager;

	/** (Optional) Externally supplied time controller */
	TSharedPtr<FMovieSceneTimeController> TimeController;

	/** When true, ignore playback replication events. */
	bool bIgnorePlaybackReplication = false;

private:

	/** The event that will be broadcast every time the sequence is updated */
	mutable FOnDaySequencePlayerUpdated OnDaySequencePlayerUpdate;

	/** The tick interval we are currently registered with (if any) */
	TOptional<FMovieSceneSequenceTickInterval> RegisteredTickInterval;

	/** The maximum tick rate prior to playing (used for overriding delta time during playback). */
	TOptional<double> OldMaxTickRate;

	/** Whether dynamic resolution frame time budget is being overridden. */
	bool bOverridingDynResFrameTimeBudget = false;

	/**
	* The last world game time at which we were ticked. Game time used is dependent on bTickEvenWhenPaused
	* Valid only if we've been ticked at least once since having a tick interval
	*/
	TOptional<float> LastTickGameTimeSeconds;

	/** Pre and post evaluation callbacks, for async evaluations */
	DECLARE_DELEGATE(FOnEvaluationCallback);
	TArray<FOnEvaluationCallback> PreEvaluationCallbacks;
	TArray<FOnEvaluationCallback> PostEvaluationCallbacks;

public:

	/**
	 * Initialize the player.
	 *
	 * @param InDaySequence The DaySequence to play.
	 * @param Owner The day sequence actor that owns this player
	 * @param Settings The desired playback settings
	 */
	UE_API void Initialize(UDaySequence* InDaySequence, ADaySequenceActor* Owner, const FMovieSceneSequencePlaybackSettings& Settings);

	UE_API void Tick(float DeltaSeconds);

	// IMovieScenePlayer interface
	UE_API virtual UObject* GetPlaybackContext() const override;

	UE_API void RewindForReplay();

private:

	/** The owning Day Sequence Actor that created this player */
	TWeakObjectPtr<ADaySequenceActor> WeakOwner;

	UPROPERTY(Transient)
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;
	
	TSharedPtr<FMovieSceneEntitySystemRunner> Runner;

	/* Set by FOverrideUpdateIntervalHandle */
	unsigned OverrideUpdateIntervalRequesterCount = 0;
	
	float DesiredUpdateInterval = 0.f;
	float DesiredBudgetMs = 0.f;
	bool bUpdateWhenPaused = false;

	/** The value of UWorld::GetUnpausedTimeSeconds last time this player was evaluated */
	float LastUnpausedTimeSeconds = -1;
	/** The value of UWorld::GetTimeSeconds last time this player was evaluated */
	float LastTimeSeconds = -1;

	bool bHasEverEvaluated = false;
};

#undef UE_API
