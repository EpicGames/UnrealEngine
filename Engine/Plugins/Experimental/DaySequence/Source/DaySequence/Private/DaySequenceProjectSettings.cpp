// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceProjectSettings.h"
#include "MovieSceneFwd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceProjectSettings)

UDaySequenceProjectSettings::UDaySequenceProjectSettings()
	: bDefaultLockEngineToDisplayRate(false)
	, DefaultDisplayRate("30fps")
	, DefaultTickResolution("24000fps")
	, DefaultClockSource(EUpdateClockSource::Tick)
{ }


void UDaySequenceProjectSettings::PostInitProperties()
{
	Super::PostInitProperties(); 

#if WITH_EDITOR
	if(IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif
}

#if WITH_EDITOR

void UDaySequenceProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}

#endif


