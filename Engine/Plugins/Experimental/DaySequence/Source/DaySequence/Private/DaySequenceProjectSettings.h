// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "DaySequenceProjectSettings.generated.h"

enum class EUpdateClockSource : uint8;

// Settings for Day sequences
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Day Sequence"))
class UDaySequenceProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UDaySequenceProjectSettings();

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="DaySequence.DefaultLockEngineToDisplayRate", Tooltip="0: Playback locked to playback frames\n1: Unlocked playback with sub frame interpolation"))
	bool bDefaultLockEngineToDisplayRate;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="DaySequence.DefaultDisplayRate", Tooltip="Specifies default display frame rate for newly created Day sequences; also defines frame locked frame rate where sequences are set to be frame locked. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."))
	FString DefaultDisplayRate;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="DaySequence.DefaultTickResolution", Tooltip="Specifies default tick resolution for newly created Day sequences. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."))
	FString DefaultTickResolution;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="DaySequence.DefaultClockSource", Tooltip="Specifies default clock source for newly created Day sequences. Examples: 0: Tick, 1: Platform, 2: Audio, 3: RelativeTimecode, 4: Timecode, 5: Custom"))
	EUpdateClockSource DefaultClockSource;

	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

