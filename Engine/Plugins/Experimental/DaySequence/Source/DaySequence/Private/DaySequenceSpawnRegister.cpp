// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceSpawnRegister.h"
#include "DaySequenceModule.h"

#include "IMovieSceneObjectSpawner.h"
#include "Modules/ModuleManager.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Engine/NetDriver.h"

FDaySequenceSpawnRegister::FDaySequenceSpawnRegister()
{
	FDaySequenceModule& DaySequenceModule = FModuleManager::GetModuleChecked<FDaySequenceModule>("DaySequence");
	DaySequenceModule.GenerateObjectSpawners(MovieSceneObjectSpawners);
}

UObject* FDaySequenceSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (Spawnable.GetObjectTemplate() != nullptr && Spawnable.GetObjectTemplate()->IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			
			UObject* SpawnedObject = MovieSceneObjectSpawner->SpawnObject(Spawnable, TemplateID, SharedPlaybackState);
			if (SpawnedObject)
			{
				return SpawnedObject;
			}
		}
	}

	return nullptr;
}

void FDaySequenceSpawnRegister::DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding)
{
	if (CustomSpawnableBinding)
	{
		CustomSpawnableBinding->DestroySpawnedObject(&Object);
	}
	else
	{
		for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
		{
			if (Object.IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
			{
				UE::Net::FScopedIgnoreStaticActorDestruction ScopedIgnoreDestruction; // net addressable actors normally have a destruction info created for them, we don't want that as these actors were spawned at runtime
				MovieSceneObjectSpawner->DestroySpawnedObject(Object);
				return;
			}
		}
	}

	checkf(false, TEXT("No valid object spawner found to destroy spawned object of type %s"), *Object.GetClass()->GetName());
}

#if WITH_EDITOR

bool FDaySequenceSpawnRegister::CanSpawnObject(UClass* InClass) const
{
	for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (InClass->IsChildOf(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			return true;
		}
	}
	return false;
}

#endif
