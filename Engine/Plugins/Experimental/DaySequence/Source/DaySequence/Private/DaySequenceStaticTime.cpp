// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceStaticTime.h"

#include "DaySequenceModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceStaticTime)

namespace UE::DaySequence
{
	void FStaticTimeManager::AddStaticTimeContributor(const FStaticTimeContributor& NewContributor)
	{
		if (!NewContributor.GetStaticTime || !NewContributor.WantsStaticTime || !NewContributor.UserObject.Get())
		{
			// Should be an ensure probably, we can't accept contributors that don't fulfill this requirement
			return;
		}

		// We remove an existing matching contributor if necessary, making sure to keep PriorityGroupSizes up to date.
		Contributors.RemoveAll([this, NewContributor](const FStaticTimeContributor& Contributor)
		{
			if (NewContributor.UserObject == Contributor.UserObject)
			{
				PriorityGroupSizes[NewContributor.Priority]--;
				return true;
			}
			
			return false;
		});

		// Add the new contributor.
		Contributors.Add(NewContributor);
		
		// Sort the array in descending order by priority for efficient group processing.
		Contributors.Sort([](const FStaticTimeContributor& LHS, const FStaticTimeContributor& RHS)
			{ return LHS.Priority > RHS.Priority; });

		// Increment (or set to 1) the priority group size.
		PriorityGroupSizes.FindOrAdd(NewContributor.Priority, 0)++;
	}

	void FStaticTimeManager::RemoveStaticTimeContributor(const UObject* UserObject)
	{
		// Get the index of the contributor to remove.
		int32 RemoveIdx = Contributors.IndexOfByPredicate([UserObject](const FStaticTimeContributor& Contributor)
		{
			return Contributor.UserObject.Get() == UserObject;
		});

		// If contributor actually exists, decrement its group count and remove it.
		if (RemoveIdx != INDEX_NONE)
		{
			PriorityGroupSizes[Contributors[RemoveIdx].Priority]--;

			Contributors.RemoveAt(RemoveIdx, EAllowShrinking::No);
		}
	}
	
	bool FStaticTimeManager::HasStaticTime(EStaticTimeChannel InChannels) const
	{
		for (const FStaticTimeContributor& Contributor : Contributors)
		{
			if (Contributor.UserObject.Get() &&
				EnumHasAnyFlags(InChannels, Contributor.Channels) &&
				Contributor.WantsStaticTime())
			{
				return true;
			}
		}

		// Reset the blend data when static time goes away.
		ResetBlendState(InChannels);

		return false;
	}
	
	float FStaticTimeManager::GetStaticTime(float InitialTime, float DayLength, const EStaticTimeChannel InChannels) const
	{
		FStaticTimeChannelData& Channel = Channels.FindOrAdd(InChannels); 
		
		float AccumulatedWeight = 0.f;
		float AccumulatedTime = 0.f;
		
		// Process batches of contributors based on priority
		for (int32 PriorityGroupStartIdx = 0; PriorityGroupStartIdx < Contributors.Num();)
		{
			// Compute the index of the final element in this priority group
			const int32 CurrentPriority = Contributors[PriorityGroupStartIdx].Priority;
			const int32 GroupSize = PriorityGroupSizes.FindChecked(CurrentPriority);	// checked because if this isn't valid something is very wrong
			const int32 PriorityGroupEndIdx = PriorityGroupStartIdx + GroupSize - 1;
			
			// Process this group 
			const FStaticTimeInfo GroupInfo = ProcessPriorityGroup(InChannels, PriorityGroupStartIdx, PriorityGroupEndIdx);

			const float EffectiveGroupWeight = (1.f - AccumulatedWeight) * GroupInfo.BlendWeight;
			const float EffectiveGroupTime = EffectiveGroupWeight * GroupInfo.StaticTime;

			AccumulatedWeight += EffectiveGroupWeight;
			AccumulatedTime += EffectiveGroupTime;

			// Advance to next priority group
			PriorityGroupStartIdx = PriorityGroupEndIdx + 1;
		}

		// The implementation below aims to provide the shortest Lerp path from the InitialTime to
		// the TargetTime based on the AccumulatedWeight.
		//
		// For a static global time (InitialTime), the formula is straightforward. We compute a
		// InitialTime relative to TargetTime to identify the BlendDelta. BlendDelta defines the range
		// of our Lerp and will exist in the range [-0.5D, 0.5D], where D = DayLength. Then we can
		// simply Lerp(InitialTime + BlendDelta, TargetTime, AccumulatedWeight).
		//
		// For a dynamic global time (ie. advancing global time), the solution is more involved. With
		// dynamic InitialTime, the Lerp range is continuously changing and may cross the [-0.5D, 0.5D]
		// boundary. This boundary is discontinuous and can result in unsightly pops in the blended
		// static time when AccumulatedWeight < 1.f.
		//
		// One method of solving that is to continuously add +/-1D in the direction that time is travelling
		// when crossing the [-0.5D, 0.5D] boundary. This would effectively result in a Lerp range of
		// [TargetTime, InitialTime +/- N*DayCycle] depending on the accumulated passage of time. One
		// consequence of that approach however is that since the Lerp now spans multiple DayCycles,
		// walking between ends of the blend region will cause up to N DayCycles to wind/unwind.
		//
		// The solution used here accounts for the above problem by leveraging the circular nature
		// of day cycles, at the cost of a discontinuity after 1D of global time has passed while
		// inside the blended region.
		//
		// Specifically, we re-frame the problem space as [-1D, 1D]
		//
		//    |--------|--------|--------|--------|
		//   -1D     -0.5D     Tgt      0.5D      1D
		//
		// We want to solve for a continuous Lerp through the above [-1D, 1D] space:
		//
		//    Lerp(StartTime, EndTime, AccumulatedWeight)
		//    StartTime = InitialTime + BlendDelta
		//
		// If we focus on the forward advancement of time case:
		//
		// 1. When we cross the +0.5D mark, BlendDelta will flip from +12 to -12. To maintain a continuity
		//    we add a BlendOffset of 1D:
		//
		//    StartTime = InitialTime + BlendDelta + BlendOffset * DayCycle
		//    BlendOffset = 1
		//
		//    This allows StartTime to continuously increment from +12 --> +13 and all the way up to +24
		//    at the 1D mark.
		//
		// 2. When we cross the 1D mark, we mirror the problem to the negative space since 1D == -1D.
		//    We do this by incrementing our BlendOffset with a Wrapped range of [-1,1].
		//
		// 3. It should be noted that this mirrored problem to the negative space can result in a visible
		//    discontinuity of up to 0.5D since the mirrored space inverts the shortest path to our target time.
		//
		// NOTE: We have tried a number of approaches to resolve the above discontinuity including inverting the lerp
		//       direction. While inverting the lerp direction ensures a continuous blend as global time passes, it
		//       results in a discontinuity when leaving the blended region for that entire day cycle. Since the
		//       lerp is inverted, W=1 would approach InitialTime rather than TargetTime and vice versa creating
		//       a discontinuity equal to (TargetTime - InitialTime).
		//
		//       Ultimately, We made the conscious decision that it is better to ensure a smooth transition into
		//       and out of the blended region over a continuous blend inside the blended region as global time
		//       passes. Since our blend state data resets on full entry into the volume or exit out of the volume
		//       the discontinuity will only show if a player remains in the blended region for a full day cycle.
		//    
		// The same applies to the backwards advancement of time case just in reverse. So instead of adding
		// a BlendOffset of 1D, we subtract.
		//
		const float HalfDayLength = 0.5f * DayLength;
		float TargetTime = AccumulatedWeight > 0.f ? AccumulatedTime / AccumulatedWeight : InitialTime;
		
		// Returns the shortest delta to TargetTime in the range [TargetTime - HalfDayLength, TargetTime + HalfDayLength]
		auto GetBlendDelta = [TargetTime, InitialTime, DayLength, HalfDayLength]()
		{
			float TimeDelta = InitialTime - TargetTime;
			if (TimeDelta > HalfDayLength)
			{
				TimeDelta -= DayLength;
			}
			else if(TimeDelta < -HalfDayLength)
			{
				TimeDelta += DayLength;
			}
			return TimeDelta;
		};

		// Returns 1 or -1 if BlendDelta is on the positive or negative side of TargetTime, respectively.
		auto GetBlendDirection = [](float Delta)
		{
			return Delta >= 0.f ? 1 : -1;
		};

		const float BlendDelta = GetBlendDelta();
		const int BlendDirection = GetBlendDirection(BlendDelta);

		// Compute LastBlendOffset based on the number of times (incl. direction) BlendDelta
		// has crossed the TargetTime +/- HalfDayLength mark.
		if (Channel.LastBlendDelta.IsSet() &&
			Channel.LastBlendDirection.IsSet() &&
			Channel.LastBlendDirection.GetValue() != BlendDirection)
		{
			const float BlendDeltaChange = BlendDelta - Channel.LastBlendDelta.GetValue();
			if (FMath::Abs(BlendDeltaChange) > HalfDayLength)
			{
				// Crossed TargetTime +- HalfDayLength
				
				// When crossing the TargetTime +- HalfDayLength, the sign is inverted from the BlendDeltaChange
				// due to the wraparound effect.
				const int BlendDeltaSign = BlendDeltaChange < 0.f ? 1 : -1;
				
				// Record our winding counts based on direction of the crossover.
				Channel.LastBlendWinding += BlendDeltaSign;
				Channel.LastBlendOffset = FMath::WrapExclusive(Channel.LastBlendOffset + BlendDeltaSign, -1, 2);
			}
			else
			{
				// Crossed TargetTime
				if (Channel.LastBlendOffset != 0)
				{
					// Loop TargetTime + HalfDayLength  <--> TargetTime - HalfDayLength
					const int BlendDeltaSign = BlendDeltaChange > 0.f ? 1 : -1;
					Channel.LastBlendOffset = FMath::WrapExclusive(Channel.LastBlendOffset + BlendDeltaSign, -1, 2);
				}
			}
		}

		// Blend against initial value if necessary
		if (AccumulatedWeight < 1.0f)
		{
			float BlendOffset = Channel.LastBlendOffset * DayLength;
			const float StartTime = TargetTime + BlendDelta + BlendOffset;

			// Wrap is inclusive of the upper bound. Ensure we wrap around correctly.
			AccumulatedTime = FMath::Lerp(StartTime, TargetTime, AccumulatedWeight);
			AccumulatedTime = FMath::Wrap(AccumulatedTime, 0.f, DayLength);
			if (AccumulatedTime >= DayLength)
			{
				AccumulatedTime -= DayLength;
			}

#if !NO_LOGGING
			UE_LOGF(LogDaySequence, VeryVerbose, "BlendDelta: %f | BlendDir: %d | BlendWinding: %d | BlendOffset: %f | StartTime: %f | InitialTime: %f | TargetTime: %f", BlendDelta, BlendDirection, Channel.LastBlendWinding, BlendOffset, StartTime, InitialTime, TargetTime);
			UE_LOGF(LogDaySequence, VeryVerbose, "AccumTime: %f | AccumWeight: %f", AccumulatedTime, AccumulatedWeight);
#endif
		}
		else
		{
			ResetBlendState(InChannels);
		}
		
		// we could probably ensure(AccumulatedWeight is nearly equal to 1.f), leaving out for now

		Channel.LastBlendDelta = BlendDelta;
		Channel.LastBlendDirection = BlendDirection;
		
		return AccumulatedTime;
	}

	float FStaticTimeManager::GetTargetStaticTime(const float InitialTime, EStaticTimeChannel InChannels) const
	{
		// Process batches of contributors based on priority
		for (int32 PriorityGroupStartIdx = 0; PriorityGroupStartIdx < Contributors.Num();)
		{
			// Compute the index of the final element in this priority group
			const int32 CurrentPriority = Contributors[PriorityGroupStartIdx].Priority;
			const int32 GroupSize = PriorityGroupSizes.FindChecked(CurrentPriority);	// checked because if this isn't valid something is very wrong
			const int32 PriorityGroupEndIdx = PriorityGroupStartIdx + GroupSize - 1;
			
			// Process this group 
			const FStaticTimeInfo GroupInfo = ProcessPriorityGroup(InChannels, PriorityGroupStartIdx, PriorityGroupEndIdx, 1.0f);

			// Return the highest priority group that contains an active contributor
			if (GroupInfo.BlendWeight > 0.f)
			{
				return GroupInfo.StaticTime;
			}

			// Advance to next priority group
			PriorityGroupStartIdx = PriorityGroupEndIdx + 1;
		}

		return InitialTime;
	}

	void FStaticTimeManager::ResetBlendState(const EStaticTimeChannel InChannels) const
	{
		FStaticTimeChannelData& Channel = Channels.FindOrAdd(InChannels);
		Channel.LastBlendDirection.Reset();
		Channel.LastBlendDelta.Reset();
		Channel.LastBlendWinding = 0;
		Channel.LastBlendOffset = 0;
	}


	FStaticTimeInfo FStaticTimeManager::ProcessPriorityGroup(const EStaticTimeChannel InChannels, int32 StartIdx, int32 EndIdx, TOptional<float> OverrideWeight) const
	{
		FStaticTimeInfo GroupInfo = {0.f, 0.f};
		int32 GroupContributors = 0;
		
		for (int32 CurrentIdx = StartIdx; CurrentIdx <= EndIdx; ++CurrentIdx)
		{
			const FStaticTimeContributor& CurrentContributor = Contributors[CurrentIdx];

			if (!CurrentContributor.UserObject.Get() ||
				!EnumHasAnyFlags(InChannels, CurrentContributor.Channels) ||
				!CurrentContributor.WantsStaticTime())
			{
				continue;
			}

			// Only increment for active contributors
			// Note: Because removing a contributor results in a discrete change in an integer value, we get pops when contributor in a group of >1 contributors has a non-1 weight.
			GroupContributors++;
			
			FStaticTimeInfo ContributorInfo;
			CurrentContributor.GetStaticTime(ContributorInfo);

			// Accumulate contributor info (we will later divide these sums once we know how many contributors there are)
			GroupInfo.BlendWeight += OverrideWeight.IsSet() ? OverrideWeight.GetValue() : ContributorInfo.BlendWeight;
			GroupInfo.StaticTime += ContributorInfo.StaticTime;
		}

		// Compute the average (before this point we have just summed all weights and times) for this group
		// If GroupContributors is 0 we treat it as 1 because that implies the values in GroupInfo are just 0.
		GroupInfo.BlendWeight /= FMath::Max(GroupContributors, 1);
		GroupInfo.StaticTime /= FMath::Max(GroupContributors, 1);
		
		return GroupInfo;
	}
}

UDaySequenceStaticTimeContributor::UDaySequenceStaticTimeContributor()
: BlendWeight(1.f)
, StaticTime(0.f)
, bWantsStaticTime(true)
, TargetActor(nullptr)
{}

void UDaySequenceStaticTimeContributor::BeginDestroy()
{
	UnbindFromDaySequenceActor();
	
	Super::BeginDestroy();
}

void UDaySequenceStaticTimeContributor::BindToDaySequenceActor(ADaySequenceActor* InTargetActor, int32 Priority)
{
	UnbindFromDaySequenceActor();

	if (!InTargetActor)
	{
		return;
	}
		
	TargetActor = InTargetActor;

	UObject* Outer = GetOuter();
	auto WantsStaticTime = [this, Outer]()
	{
		return IsValidChecked(this) && IsValid(Outer) && bWantsStaticTime;
	};

	auto GetStaticTime = [this, WantsStaticTime](UE::DaySequence::FStaticTimeInfo& OutRequest)
	{
		if (WantsStaticTime())
		{
			OutRequest.BlendWeight = BlendWeight;
			OutRequest.StaticTime = StaticTime;
			return true;
		}

		return false;
	};

	TargetActor->RegisterStaticTimeContributor({ Outer, Priority, WantsStaticTime, GetStaticTime });
}

void UDaySequenceStaticTimeContributor::UnbindFromDaySequenceActor()
{
	if (!TargetActor)
	{
		return;
	}

	TargetActor->UnregisterStaticTimeContributor(this);

	TargetActor = nullptr;
}
