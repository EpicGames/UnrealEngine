// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Algo/Sort.h"
#include "DaySequenceActor.h"

#include "DaySequenceStaticTime.generated.h"

namespace UE::DaySequence
{
	// The fundamental piece of data used by the static time system.
	struct FStaticTimeInfo
	{
		float BlendWeight = 0.f;
		float StaticTime = 0.f;
	};

	// The function signature for the static time requested callback. Must return true for a contributor to be considered
	using FWantsStaticTimeFunction = TFunction<bool()>;
	
	// The function signature for the static time info getter callback. Only called if WantsStaticTime returns true.
	using FGetStaticTimeFunction = TFunction<bool(FStaticTimeInfo& OutRequest)>;

	enum class EStaticTimeChannel
	{
		None			= 0,
		FixedTime		= 1 << 0,
		LocalFixedTime	= 1 << 1
	};
	ENUM_CLASS_FLAGS(EStaticTimeChannel);

	// Contributors register an instance of this struct in order to request a static time.
	struct FStaticTimeContributor
	{
		// Determines the lifetime of the object and prevents double registration.
		TWeakObjectPtr<UObject> UserObject;

		// Used for sorting contributors
		int32 Priority;

		// Returns whether or not this contributor is active.
		FWantsStaticTimeFunction WantsStaticTime;

		// Provides caller with desired static time information.
		FGetStaticTimeFunction GetStaticTime;

		// The channel(s) that this static time contributor affects.
		EStaticTimeChannel Channels = EStaticTimeChannel::FixedTime;
	};

	struct FStaticTimeChannelData
	{
		// Cached blend data
		int LastBlendWinding = 0;
		int LastBlendOffset = 0;
		TOptional<float> LastBlendDelta;
		TOptional<int> LastBlendDirection;
	};
	
	struct FStaticTimeManager
	{
		void AddStaticTimeContributor(const FStaticTimeContributor& NewContributor);
		void RemoveStaticTimeContributor(const UObject* UserObject);
		bool HasStaticTime(EStaticTimeChannel InChannels = EStaticTimeChannel::FixedTime) const;
		float GetStaticTime(float InitialTime, float DayLength, EStaticTimeChannel InChannels = EStaticTimeChannel::FixedTime) const;
		float GetTargetStaticTime(float InitialTime, EStaticTimeChannel InChannels) const;
		
	private:
		void ResetBlendState(EStaticTimeChannel InChannels) const;
		
		// Handles multiple contributors of the same priority by averaging their weights and times and returning a single desired weight and time
		FStaticTimeInfo ProcessPriorityGroup(EStaticTimeChannel InChannels, int32 StartIdx, int32 EndIdx, TOptional<float> OverrideWeight = TOptional<float>()) const;

		TArray<FStaticTimeContributor> Contributors;

		// We precompute this when mutating the Contributors array to avoid redundant reiteration over Contributors when computing static time
		TMap<int32, int32> PriorityGroupSizes;

		// Mutable because cached data is set in (Has/Get)StaticTime.
		mutable TMap<EStaticTimeChannel, FStaticTimeChannelData> Channels;
	};
}

/**
 * A Blueprint exposed static time contributor.
 * Used to contribute to static time blending for the specified Day Sequence Actor without needing to spawn actors and/or components.
 */
UCLASS(BlueprintType)
class UDaySequenceStaticTimeContributor : public UObject
{
	GENERATED_BODY()

public:

	UDaySequenceStaticTimeContributor();
	
	virtual void BeginDestroy() override;
	
	/** The desired blend weight. Once bound to a Day Sequence Actor, this can be freely changed without rebinding. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Day Sequence")
	float BlendWeight;

	/** The desired static time. Once bound to a Day Sequence Actor, this can be freely changed without rebinding. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Day Sequence")
	float StaticTime;

	/** Determines whether or not this contributor is effective once we are bound. This can be freely changed to enable/disable the contributor without rebinding. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Day Sequence")
	bool bWantsStaticTime;

	/** Begin contributing static time to the specified actor. */
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	void BindToDaySequenceActor(ADaySequenceActor* InTargetActor, int32 Priority = 1000);

	/** Stop contributing static time. */
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	void UnbindFromDaySequenceActor();
	
private:

	UPROPERTY(Transient)
	TObjectPtr<ADaySequenceActor> TargetActor;
};	
