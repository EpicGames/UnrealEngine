// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceSubsystem.h"

#include "DaySequenceCheatManagerExtension.h"
#include "DaySequenceActor.h"
#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceSubsystem)

namespace UE::DaySequence
{
	static TAutoConsoleVariable<bool> CVarEnableCheats(
		TEXT("DaySequence.EnableCheats"),
		true,
		TEXT("When true, Day Sequence cheats will be enabled."),
		ECVF_Default
	);
}

bool UDaySequenceSubsystem::DoesSupportWorldType(EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Game || WorldType == EWorldType::Editor || WorldType == EWorldType::PIE;
}

void UDaySequenceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}

void UDaySequenceSubsystem::Deinitialize()
{
	Super::Deinitialize();
}

ADaySequenceActor* UDaySequenceSubsystem::GetDaySequenceActor(bool bFindFallbackOnNull) const
{
	if (ADaySequenceActor* CurrentDaySequenceActor = DaySequenceActor.Get())
	{
		return CurrentDaySequenceActor;
	}

	if (bFindFallbackOnNull)
	{
		// Fallback to iterating over all DaySequenceActors in the world and returning
		// the first match.
		TActorIterator<ADaySequenceActor> It(GetWorld());
		return It ? *It : nullptr;
	}

	return nullptr;
}

void UDaySequenceSubsystem::SetDaySequenceActor(ADaySequenceActor* InActor)
{
	DaySequenceActor = InActor;
	BroadcastOnDaySequenceActorSet(InActor);
}

void UDaySequenceSubsystem::OnCheatManagerCreated(UCheatManager* CheatManager)
{
	if (UE::DaySequence::CVarEnableCheats.GetValueOnAnyThread())
	{
		CheatManagerExtension = NewObject<UDaySequenceCheatManagerExtension>(CheatManager);
		CheatManager->AddCheatManagerExtension(CheatManagerExtension.Get());
	}

	UE::DaySequence::CVarEnableCheats.AsVariable()->OnChangedDelegate().AddWeakLambda(this, [this, CheatManager](const IConsoleVariable* InConsoleVariable)
	{
		if (InConsoleVariable->GetBool())
		{
			// Cheats enabled

			if (CheatManagerExtension.Get() || !IsValid(CheatManager))
			{
				return;
			}
			
			CheatManagerExtension = NewObject<UDaySequenceCheatManagerExtension>(CheatManager);
			CheatManager->AddCheatManagerExtension(CheatManagerExtension.Get());
		}
		else
		{
			// Cheats disabled

			if (!CheatManagerExtension.Get() || !IsValid(CheatManager))
			{
				return;
			}

			CheatManager->RemoveCheatManagerExtension(CheatManagerExtension.Get());
			CheatManagerExtension.Reset();
		}
	});
}

void UDaySequenceSubsystem::BroadcastOnDaySequenceActorSet(ADaySequenceActor* InActor) const
{
	OnDaySequenceActorSet.Broadcast(InActor);
	OnDaySequenceActorSetEvent.Broadcast(InActor);
}