// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceTime)

FDaySequenceTime::FDaySequenceTime(int32 InHours, int32 InMinutes, int32 InSeconds)
: Hours(InHours)
, Minutes(InMinutes)
, Seconds(InSeconds)
{}

bool FDaySequenceTime::operator==(const FDaySequenceTime& RHS) const
{
	return Hours == RHS.Hours && Minutes == RHS.Minutes && Seconds == RHS.Seconds;
}

bool FDaySequenceTime::operator!=(const FDaySequenceTime& RHS) const
{
	return !operator==(RHS);
}

float FDaySequenceTime::ToHours() const
{
	return ToSeconds() / SecondsPerHour;
}

float FDaySequenceTime::ToSeconds() const
{
	return Hours * SecondsPerHour + Minutes * SecondsPerMinute + Seconds;
}

FDaySequenceTime FDaySequenceTime::FromHours(float InHours)
{
	return FromSeconds(InHours * SecondsPerHour);
}

FDaySequenceTime FDaySequenceTime::FromSeconds(float InSeconds)
{
	float Seconds = InSeconds;
	const float Hours = FMath::Floor(InSeconds / SecondsPerHour);
	Seconds -= Hours * SecondsPerHour;
	const float Minutes = FMath::Floor(Seconds / SecondsPerMinute);
	Seconds -= Minutes * SecondsPerMinute;

	return FDaySequenceTime(Hours, Minutes, FMath::Floor(Seconds));
}
