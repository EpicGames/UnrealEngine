// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceTrack.h"
#include "Sections/MovieSceneSubSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceTrack)

#define LOCTEXT_NAMESPACE "DaySequenceTrack"

UDaySequenceTrack::UDaySequenceTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
#if WITH_EDITORONLY_DATA
	DisplayName = LOCTEXT("TrackName", "Subsequences");
#endif
}

UMovieSceneSection* UDaySequenceTrack::CreateNewSection()
{
	// Force all sections created to inherit object flags from its parent track.
	// TODO: Consider making an AddSequence interface with explicit ObjectFlags support for UMovieSceneSubTrack.
	return NewObject<UMovieSceneSubSection>(this, NAME_None, GetFlags());
}

#if WITH_EDITORONLY_DATA
FText UDaySequenceTrack::GetDisplayName() const
{
	return DisplayName;
}
#endif

#undef LOCTEXT_NAMESPACE

