// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralDaySequence.h"

#include "DaySequence.h"
#include "DaySequenceActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralDaySequence)

#define LOCTEXT_NAMESPACE "ProceduralDaySequence"

UDaySequence* FProceduralDaySequence::GetSequence(ADaySequenceActor* InActor)
{
	WeakTargetActor = InActor;
	
	UDaySequence* ProceduralSequence = nullptr;
	
	if (InActor)
	{
		UProceduralDaySequenceBuilder* SequenceBuilder = NewObject<UProceduralDaySequenceBuilder>();
		ProceduralSequence = SequenceBuilder->Initialize(InActor);
		BuildSequence(SequenceBuilder);
	}
	
	return ProceduralSequence;
}

#undef LOCTEXT_NAMESPACE