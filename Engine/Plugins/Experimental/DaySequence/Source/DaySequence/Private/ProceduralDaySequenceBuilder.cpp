// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralDaySequenceBuilder.h"

#include "DaySequence.h"
#include "DaySequenceActor.h"
#include "DaySequenceTime.h"

#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneDoubleSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Sections/MovieSceneColorSection.h"
#include "Sections/MovieScenePrimitiveMaterialSection.h"

#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "Tracks/MovieSceneVisibilityTrack.h"

#include "Materials/MaterialInterface.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneVisibilitySection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralDaySequenceBuilder)

#define LOCTEXT_NAMESPACE "ProceduralDaySequenceBuilder"

namespace UE::DaySequence
{
	FFrameNumber GetKeyFrameNumber(float NormalizedTime, const TRange<FFrameNumber>& FrameRange)
	{
		NormalizedTime = FMath::Clamp(NormalizedTime, 0.f, 1.f);
		const unsigned StartFrameNum = FrameRange.GetLowerBoundValue().Value;
		const unsigned EndFrameNum = FrameRange.GetUpperBoundValue().Value;
		const unsigned FrameCount = EndFrameNum - StartFrameNum;
	
		if (FMath::IsNearlyEqual(NormalizedTime, 1.f))
		{
			return FFrameNumber(static_cast<int32>(EndFrameNum - 1));
		}

		return FFrameNumber(static_cast<int32>(NormalizedTime * FrameCount + StartFrameNum));
	}

	bool IsPropertyValid(UObject* Object, FProperty* Property)
	{
		if (!Property)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Invalid property specified for object %s."), *Object->GetName()), ELogVerbosity::Error);
			return false;
		}

		if (Property->HasAnyPropertyFlags(CPF_Deprecated))
		{
			// Emit a warning for deprecated properties but still consider them valid
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Depcrecated property specified: %s for object %s."), *Property->GetName(), *Object->GetName()), ELogVerbosity::Warning);
		}

		return true;
	}

	void AddDoubleKey(FFrameNumber Time, double Value, FMovieSceneDoubleChannel* Channel, ERichCurveInterpMode InterpMode)
	{
		if (!Channel)
		{
			return;
		}
		
		switch (InterpMode)
		{
		case RCIM_Linear: Channel->AddLinearKey(Time, Value); break;
		case RCIM_Constant: Channel->AddConstantKey(Time, Value); break;
		case RCIM_Cubic: Channel->AddCubicKey(Time, Value); break;
		case RCIM_None: break;
		}
	}

	void AddFloatKey(FFrameNumber Time, double Value, FMovieSceneFloatChannel* Channel, ERichCurveInterpMode InterpMode)
	{
		if (!Channel)
		{
			return;
		}
		
		switch (InterpMode)
		{
		case RCIM_Linear: Channel->AddLinearKey(Time, Value); break;
		case RCIM_Constant: Channel->AddConstantKey(Time, Value); break;
		case RCIM_Cubic: Channel->AddCubicKey(Time, Value); break;
		case RCIM_None: break;
		}
	}
};

UDaySequence* UProceduralDaySequenceBuilder::Initialize(ADaySequenceActor* InActor, UDaySequence* InitialSequence, bool bClearInitialSequence)
{
	if (!ensureAlways(!TargetActor) || !ensureAlways(InActor))
	{
		return nullptr;
	}

	TargetActor = InActor;
	
	if (InitialSequence)
	{
		ProceduralDaySequence = InitialSequence;

		if (bClearInitialSequence)
		{
			ClearKeys();
		}
	}
	else
	{
		const FName SequenceName = MakeUniqueObjectName(InActor, UDaySequence::StaticClass());
		ProceduralDaySequence = NewObject<UDaySequence>(InActor, SequenceName, RF_Transient);
		ProceduralDaySequence->Initialize(RF_Transient);

		const float DaySeconds = TargetActor->GetTimePerCycle() * FDaySequenceTime::SecondsPerHour;
		
		const int32 Duration = ProceduralDaySequence->GetMovieScene()->GetTickResolution().AsFrameNumber(DaySeconds).Value;
		ProceduralDaySequence->GetMovieScene()->SetPlaybackRange(0, Duration);
	}

	return ProceduralDaySequence;
}

bool UProceduralDaySequenceBuilder::IsInitialized() const
{
	return IsValid(TargetActor) && IsValid(ProceduralDaySequence);
}

void UProceduralDaySequenceBuilder::SetActiveBoundObject(UObject* InObject)
{
	if (!IsValid(InObject))
	{
		FFrame::KismetExecutionMessage(TEXT("SetActiveBoundObject called with an invalid object!"), ELogVerbosity::Error);
		return;
	}

	ActiveBoundObject = InObject;

	USceneComponent* Component = Cast<USceneComponent>(InObject);
	AActor*          Actor     = Cast<AActor>(InObject);

	ADaySequenceActor* AssociatedActor = nullptr;
	if (Actor)
	{
		AssociatedActor = Cast<ADaySequenceActor>(Actor);
	}
	else if (Component)
	{
		AssociatedActor = Cast<ADaySequenceActor>(Component->GetOwner());
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("SetActiveBoundObject called with an object that is neither an Actor or a Scene Component!"), ELogVerbosity::Error);
		return;
	}

	ActiveBinding = GetOrCreateProceduralBinding(InObject);
}

void UProceduralDaySequenceBuilder::AddBoolOverride(FName PropertyName, bool Value)
{
	TPair<float, bool> A = {0.f, Value};
	TPair<float, bool> B = {1.f, Value};

	AddBoolKeys(PropertyName, {A, B});
}

void UProceduralDaySequenceBuilder::AddBoolKey(FName PropertyName, float Key, bool Value)
{
	AddBoolKey(PropertyName, TPair<float, bool>(Key, Value));
}

void UProceduralDaySequenceBuilder::AddBoolKey(FName PropertyName, const TPair<float, bool>& KeyValue)
{
	AddBoolKeys(PropertyName, TArray {KeyValue});
}

void UProceduralDaySequenceBuilder::AddBoolKeys(FName PropertyName, const TArray<TPair<float, bool>>& KeysAndValues)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddBoolKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	
	FTrackInstancePropertyBindings Bindings(PropertyName, PropertyName.ToString());
	FProperty* Property = Bindings.GetProperty(*ActiveBoundObject);
	if (!UE::DaySequence::IsPropertyValid(ActiveBoundObject, Property))
	{
		// Do nothing
	}
	else if (Property->IsA<FBoolProperty>())
	{
		UMovieSceneBoolSection* Section = CreateOrAddPropertyOverrideSection<UMovieSceneBoolTrack, UMovieSceneBoolSection>(PropertyName);
		
		for (const TPair<float, bool>& KeyValue : KeysAndValues)
		{
			const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
			
			Section->GetChannel().AddKeys(TArray {FrameNumber}, TArray {KeyValue.Value});
		}

		Section->MarkAsChanged();
	}
	else
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to animate a %s property as a bool."), *Property->GetClass()->GetName()), ELogVerbosity::Error);
	}
}

void UProceduralDaySequenceBuilder::AddScalarOverride(FName PropertyName, double Value)
{
	TPair<float, double> A = {0.f, Value};
	TPair<float, double> B = {1.f, Value};

	AddScalarKeys(PropertyName, {A, B}, RCIM_Linear);
}

void UProceduralDaySequenceBuilder::AddScalarKey(FName PropertyName, float Key, double Value, ERichCurveInterpMode InterpMode)
{
	AddScalarKey(PropertyName, TPair<float, double>(Key, Value), InterpMode);
}

void UProceduralDaySequenceBuilder::AddScalarKey(FName PropertyName, const TPair<float, double>& KeyValue, ERichCurveInterpMode InterpMode)
{
	AddScalarKeys(PropertyName, TArray {KeyValue}, InterpMode);
}

void UProceduralDaySequenceBuilder::AddScalarKeys(FName PropertyName, const TArray<TPair<float, double>>& KeysAndValues, ERichCurveInterpMode InterpMode)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddScalarKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	
	FTrackInstancePropertyBindings Bindings(PropertyName, PropertyName.ToString());
	FProperty* Property = Bindings.GetProperty(*ActiveBoundObject);
	if (!UE::DaySequence::IsPropertyValid(ActiveBoundObject, Property))
	{
		// Do nothing
	}
	else if (Property->IsA<FFloatProperty>())
	{
		UMovieSceneFloatSection* FloatSection = CreateOrAddPropertyOverrideSection<UMovieSceneFloatTrack, UMovieSceneFloatSection>(PropertyName);
		
		for (const TPair<float, double>& KeyValue : KeysAndValues)
		{
			const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
			UE::DaySequence::AddFloatKey(FrameNumber, KeyValue.Value, &FloatSection->GetChannel(), InterpMode);
		}

		FloatSection->MarkAsChanged();
	}
	else if (Property->IsA<FDoubleProperty>())
	{
		UMovieSceneDoubleSection* DoubleSection = CreateOrAddPropertyOverrideSection<UMovieSceneDoubleTrack, UMovieSceneDoubleSection>(PropertyName);
		
		for (const TPair<float, double>& KeyValue : KeysAndValues)
		{
			const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
			UE::DaySequence::AddDoubleKey(FrameNumber, KeyValue.Value, &DoubleSection->GetChannel(), InterpMode);
		}

		DoubleSection->MarkAsChanged();
	}
	else
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to animate a %s property as a scalar."), *Property->GetClass()->GetName()), ELogVerbosity::Error);
	}
}

void UProceduralDaySequenceBuilder::AddVectorOverride(FName PropertyName, FVector Value)
{
	TPair<float, FVector> A = {0.f, Value};
	TPair<float, FVector> B = {1.f, Value};

	AddVectorKeys(PropertyName, {A, B}, RCIM_Linear);
}

void UProceduralDaySequenceBuilder::AddVectorKey(FName PropertyName, float Key, FVector Value, ERichCurveInterpMode InterpMode)
{
	AddVectorKey(PropertyName, TPair<float, FVector>(Key, Value), InterpMode);
}

void UProceduralDaySequenceBuilder::AddVectorKey(FName PropertyName, const TPair<float, FVector>& KeyValue, ERichCurveInterpMode InterpMode)
{
	AddVectorKeys(PropertyName, TArray {KeyValue}, InterpMode);
}

void UProceduralDaySequenceBuilder::AddVectorKeys(FName PropertyName, const TArray<TPair<float, FVector>>& KeysAndValues, ERichCurveInterpMode InterpMode)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddVectorKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	
	FTrackInstancePropertyBindings Bindings(PropertyName, PropertyName.ToString());
	FProperty* Property = Bindings.GetProperty(*ActiveBoundObject);
	if (!UE::DaySequence::IsPropertyValid(ActiveBoundObject, Property))
	{
		// Do nothing
	}
	else if (Property->IsA<FStructProperty>() && CastField<FStructProperty>(Property)->Struct == TBaseStructure<FVector>::Get())
	{
		UMovieSceneDoubleVectorSection* VectorSection = CreateOrAddPropertyOverrideSection<UMovieSceneDoubleVectorTrack, UMovieSceneDoubleVectorSection>(PropertyName);
		VectorSection->SetChannelsUsed(3);

		FMovieSceneDoubleChannel* X = VectorSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0);
		FMovieSceneDoubleChannel* Y = VectorSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(1);
		FMovieSceneDoubleChannel* Z = VectorSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(2);
		
		for (const TPair<float, FVector>& KeyValue : KeysAndValues)
		{
			const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
			
			UE::DaySequence::AddDoubleKey(FrameNumber, KeyValue.Value.X, X, InterpMode);
			UE::DaySequence::AddDoubleKey(FrameNumber, KeyValue.Value.X, Y, InterpMode);
			UE::DaySequence::AddDoubleKey(FrameNumber, KeyValue.Value.X, Z, InterpMode);
		}

		VectorSection->MarkAsChanged();
	}
}

void UProceduralDaySequenceBuilder::AddColorOverride(FName PropertyName, FLinearColor Value)
{
	TPair<float, FLinearColor> A = {0.f, Value};
	TPair<float, FLinearColor> B = {1.f, Value};

	AddColorKeys(PropertyName, {A, B}, RCIM_Linear);
}

void UProceduralDaySequenceBuilder::AddColorKeys(FName PropertyName, const TArray<TPair<float, FLinearColor>>& KeysAndValues, ERichCurveInterpMode InterpMode)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddColorKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();

	FTrackInstancePropertyBindings Bindings(PropertyName, PropertyName.ToString());
	FProperty* Property = Bindings.GetProperty(*ActiveBoundObject);
	if (!UE::DaySequence::IsPropertyValid(ActiveBoundObject, Property))
	{
		// Do nothing
	}
	else if (Property->IsA<FStructProperty>() && 
		(CastField<FStructProperty>(Property)->Struct == TBaseStructure<FLinearColor>::Get() || CastField<FStructProperty>(Property)->Struct == TBaseStructure<FColor>::Get()) )
	{
		UMovieSceneColorSection* ColorSection = CreateOrAddPropertyOverrideSection<UMovieSceneColorTrack, UMovieSceneColorSection>(PropertyName);

		for (const TPair<float, FLinearColor>& KeyValue : KeysAndValues)
		{
			const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());

			UE::DaySequence::AddFloatKey(FrameNumber, KeyValue.Value.R, &ColorSection->GetRedChannel(), InterpMode);
			UE::DaySequence::AddFloatKey(FrameNumber, KeyValue.Value.G, &ColorSection->GetGreenChannel(), InterpMode);
			UE::DaySequence::AddFloatKey(FrameNumber, KeyValue.Value.B, &ColorSection->GetBlueChannel(), InterpMode);
			UE::DaySequence::AddFloatKey(FrameNumber, KeyValue.Value.A, &ColorSection->GetAlphaChannel(), InterpMode);
		}
		
		ColorSection->MarkAsChanged();
	}
}

void UProceduralDaySequenceBuilder::AddTransformOverride(const FTransform& Value)
{
	AddTransformKey(0.f, Value, RCIM_Linear);
	AddTransformKey(1.f, Value, RCIM_Linear);
}

void UProceduralDaySequenceBuilder::AddTransformKey(float Key, const FTransform& Value, ERichCurveInterpMode InterpMode)
{
	using namespace UE::DaySequence;
	
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddTransformKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	AddTranslationKey(Key, Value.GetLocation(), InterpMode);
	AddRotationKey(Key, Value.Rotator(), InterpMode);
	AddScaleKey(Key, Value.GetScale3D(), InterpMode);
}

void UProceduralDaySequenceBuilder::AddTranslationKey(float Key, const FVector& Value, ERichCurveInterpMode InterpMode)
{
	using namespace UE::DaySequence;
	
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddTranslationKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}

	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	const FFrameNumber FrameNumber = GetKeyFrameNumber(Key, MovieScene->GetPlaybackRange());
	const UMovieScene3DTransformSection* TransformSection = CreateOrAddPropertyOverrideSection<UMovieScene3DTransformTrack, UMovieScene3DTransformSection>("Transform");

	AddDoubleKey(FrameNumber, Value.X, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(0), InterpMode);
	AddDoubleKey(FrameNumber, Value.Y, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(1), InterpMode);
	AddDoubleKey(FrameNumber, Value.Z, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(2), InterpMode);
}

void UProceduralDaySequenceBuilder::AddRotationKey(float Key, const FRotator& Value, ERichCurveInterpMode InterpMode)
{
	using namespace UE::DaySequence;
	
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddRotationKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}

	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	const FFrameNumber FrameNumber = GetKeyFrameNumber(Key, MovieScene->GetPlaybackRange());
	const UMovieScene3DTransformSection* TransformSection = CreateOrAddPropertyOverrideSection<UMovieScene3DTransformTrack, UMovieScene3DTransformSection>("Transform");
	
	AddDoubleKey(FrameNumber, Value.Roll,  TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(3), InterpMode);
	AddDoubleKey(FrameNumber, Value.Pitch, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(4), InterpMode);
	AddDoubleKey(FrameNumber, Value.Yaw,   TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(5), InterpMode);
}

void UProceduralDaySequenceBuilder::AddScaleKey(float Key, const FVector& Value, ERichCurveInterpMode InterpMode)
{
	using namespace UE::DaySequence;
	
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddScaleKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}

	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	const FFrameNumber FrameNumber = GetKeyFrameNumber(Key, MovieScene->GetPlaybackRange());
	const UMovieScene3DTransformSection* TransformSection = CreateOrAddPropertyOverrideSection<UMovieScene3DTransformTrack, UMovieScene3DTransformSection>("Transform");

	AddDoubleKey(FrameNumber, Value.X, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(6), InterpMode);
	AddDoubleKey(FrameNumber, Value.Y, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(7), InterpMode);
	AddDoubleKey(FrameNumber, Value.Z, TransformSection->GetChannelProxy().GetChannel<FMovieSceneDoubleChannel>(8), InterpMode);
}

void UProceduralDaySequenceBuilder::AddMaterialOverride(int32 MaterialIndex, UMaterialInterface* Value)
{
	if (!IsInitialized())
    {
    	FFrame::KismetExecutionMessage(TEXT("AddMaterialOverride called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
    	return;
    }
	
	UMovieScenePrimitiveMaterialTrack* MaterialTrack = CreateOrAddOverrideTrack<UMovieScenePrimitiveMaterialTrack>(FName());
	MaterialTrack->SetMaterialInfo(FComponentMaterialInfo{ FName(), MaterialIndex, EComponentMaterialType::IndexedMaterial });

	UMovieScenePrimitiveMaterialSection* Section = Cast<UMovieScenePrimitiveMaterialSection>(MaterialTrack->GetAllSections()[0]);
	Section->MaterialChannel.SetDefault(Value);
}

void UProceduralDaySequenceBuilder::AddScalarMaterialParameterOverride(FName ParameterName, int32 MaterialIndex, float Value)
{
	TPair<float, float> A = {0.f, Value};
	TPair<float, float> B = {1.f, Value};

	AddScalarMaterialParameterKeys(ParameterName, MaterialIndex, {A, B});
}

void UProceduralDaySequenceBuilder::AddScalarMaterialParameterKeys(FName ParameterName, int32 MaterialIndex, const TArray<TPair<float, float>>& KeysAndValues)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddScalarMaterialParameterKeys called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();

	// Material parameter tracks use the material index as the unique name
	const FName IndexAsName(*FString::FromInt(MaterialIndex));
	UMovieSceneComponentMaterialTrack* MaterialTrack = CreateOrAddOverrideTrack<UMovieSceneComponentMaterialTrack>(IndexAsName);
	MaterialTrack->SetMaterialInfo(FComponentMaterialInfo{FName(), MaterialIndex, EComponentMaterialType::IndexedMaterial });

	for (const TPair<float, float>& KeyValue : KeysAndValues)
	{
		const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
		
		MaterialTrack->AddScalarParameterKey(ParameterName, FrameNumber, KeyValue.Value);
	}
}

void UProceduralDaySequenceBuilder::AddColorMaterialParameterOverride(FName ParameterName, int32 MaterialIndex, FLinearColor Value)
{
	TPair<float, FLinearColor> A = {0.f, Value};
	TPair<float, FLinearColor> B = {1.f, Value};

	AddColorMaterialParameterKeys(ParameterName, MaterialIndex, {A, B});
}

void UProceduralDaySequenceBuilder::AddColorMaterialParameterKeys(FName ParameterName, int32 MaterialIndex, const TArray<TPair<float, FLinearColor>>& KeysAndValues)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddColorMaterialParameterKeys called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();

	// Material parameter tracks use the material index as the unique name
	const FName IndexAsName(*FString::FromInt(MaterialIndex));
	UMovieSceneComponentMaterialTrack* MaterialTrack = CreateOrAddOverrideTrack<UMovieSceneComponentMaterialTrack>(IndexAsName);
	MaterialTrack->SetMaterialInfo(FComponentMaterialInfo{FName(), MaterialIndex, EComponentMaterialType::IndexedMaterial });

	for (const TPair<float, FLinearColor>& KeyValue : KeysAndValues)
	{
		const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
		
		MaterialTrack->AddColorParameterKey(ParameterName, FrameNumber, KeyValue.Value);
	}
}

void UProceduralDaySequenceBuilder::AddVisibilityOverride(bool bValue)
{
	TPair<float, bool> A = {0.f, bValue};
	TPair<float, bool> B = {1.f, bValue};

	AddVisibilityKeys({A, B});
}

void UProceduralDaySequenceBuilder::AddVisibilityKeys(const TArray<TPair<float, bool>>& KeysAndValues)
{
	if (!IsInitialized())
	{
		FFrame::KismetExecutionMessage(TEXT("AddVisibilityKey(s) called on an uninitialized Procedural Day Sequence Builder!"), ELogVerbosity::Error);
		return;
	}
	
	const UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	
	static const FName ActorVisibilityTrackName = TEXT("bHidden");
	static const FName ComponentVisibilityTrackName = TEXT("bHiddenInGame");
	
	const bool bIsComponent = ActiveBoundObject->IsA<USceneComponent>();
	const bool bIsActor     = ActiveBoundObject->IsA<AActor>();
	
	if (!bIsComponent && !bIsActor)
	{
		FFrame::KismetExecutionMessage(TEXT("AddVisibilityKey(s) called but ActiveBoundObject is neither an Actor nor a Scene Component!"), ELogVerbosity::Error);
		return;
	}

	// We can check just bIsComponent because we early either if _both_ are false, so it must be one or the other.
	const FName& TrackName = bIsComponent ? ComponentVisibilityTrackName : ActorVisibilityTrackName;
	UMovieSceneVisibilitySection* VisibilitySection = CreateOrAddPropertyOverrideSection<UMovieSceneVisibilityTrack, UMovieSceneVisibilitySection>(TrackName);

	for (const TPair<float, bool>& KeyValue : KeysAndValues)
	{
		const FFrameNumber FrameNumber = UE::DaySequence::GetKeyFrameNumber(KeyValue.Key, MovieScene->GetPlaybackRange());
		
		VisibilitySection->GetChannel().AddKeys(TArray {FrameNumber}, TArray {KeyValue.Value});
	}
	
	VisibilitySection->MarkAsChanged();
}

void UProceduralDaySequenceBuilder::ClearKeys()
{
	if (!ProceduralDaySequence)
	{
		return;
	}
	
	if (UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene())
	{
		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			// Inconvenient we have to do this but at least FindBinding is doing a binary search and we do this once per binding.
			if (FMovieSceneBinding* MutableBinding = MovieScene->FindBinding(Binding.GetObjectGuid()))
			{
				// We have to copy the array here because we are mutating the internal array
				for (TArray<UMovieSceneTrack*> Tracks = MutableBinding->GetTracks(); UMovieSceneTrack* Track : Tracks)
				{
					MutableBinding->RemoveTrack(*Track, MovieScene);
				}
			}
		}

		MovieScene->MarkAsChanged();
	}
}



FGuid UProceduralDaySequenceBuilder::GetOrCreateProceduralBinding(UObject* Object) const
{
	if (!Object)
	{
		FFrame::KismetExecutionMessage(TEXT("Null Object parameter specified."), ELogVerbosity::Error);
		return FGuid();
	}

	USceneComponent* Component = Cast<USceneComponent>(Object);
	AActor*          Actor     = Cast<AActor>(Object);

	if (!TargetActor)
	{
		FFrame::KismetExecutionMessage(TEXT("No valid ADaySequenceActor set. Have you called SetActiveBoundObject yet?"), ELogVerbosity::Error);
		return FGuid();
	}

	check(ProceduralDaySequence);

	UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();

	// Find the main binding
	TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(TargetActor, ProceduralDaySequence);
	FGuid RootGuid = ProceduralDaySequence->FindBindingFromObject(TargetActor, SharedPlaybackState);
	if (!RootGuid.IsValid())
	{
		// Explicitly invoke MarkAsChanged to ensure proper notification at runtime.
		// The Modify that AddPossessable invokes only works in editor.
		MovieScene->MarkAsChanged();

		RootGuid = MovieScene->AddPossessable(TargetActor->GetName(), TargetActor->GetClass());
		ProceduralDaySequence->BindPossessableObject(RootGuid, *TargetActor, TargetActor);
	}

	// If we're trying to animate the actor, just return the root binding
	if (Actor)
	{
		return RootGuid;
	}

	// If we're trying to animate a component within the actor, retrieve or create a child binding for that
	FGuid ComponentGuid = ProceduralDaySequence->FindBindingFromObject(Component, SharedPlaybackState);
	if (!ComponentGuid.IsValid() && Component)
	{
		// Explicitly invoke MarkAsChanged to ensure proper notification at runtime.
		// The Modify that AddPossessable invokes only works in editor.
		MovieScene->MarkAsChanged();

		ComponentGuid = MovieScene->AddPossessable(Component->GetName(), Component->GetClass());
		FMovieScenePossessable* NewPossessable = MovieScene->FindPossessable(ComponentGuid);
		NewPossessable->SetParent(RootGuid, MovieScene);

		ProceduralDaySequence->BindPossessableObject(ComponentGuid, *Component, TargetActor);
	}

	return ComponentGuid;
}

template<typename TrackType>
TrackType* UProceduralDaySequenceBuilder::CreateOrAddOverrideTrack(FName Name)
{
	UMovieScene* MovieScene = ProceduralDaySequence->GetMovieScene();
	TrackType* Track = MovieScene->FindTrack<TrackType>(ActiveBinding, Name);
	if (!Track)
	{
		// Clear RF_Transactional and set RF_Transient on created tracks and sections
		// to avoid dirtying the package for these procedurally generated sequences.
		// RF_Transactional is explicitly set in UMovieSceneSection/Track::PostInitProperties.
		Track = NewObject<TrackType>(MovieScene, NAME_None, RF_Transient);
		Track->ClearFlags(RF_Transactional);

		UMovieSceneSection* Section = Track->CreateNewSection();
		Section->ClearFlags(RF_Transactional);
		Section->SetFlags(RF_Transient);
		Section->SetRange(TRange<FFrameNumber>::All());

		Track->AddSection(*Section);
		MovieScene->AddGivenTrack(Track, ActiveBinding);
	}

	return Track;
}

template<typename TrackType>
TrackType* UProceduralDaySequenceBuilder::CreateOrAddPropertyOverrideTrack(FName InPropertyName)
{
	TrackType* Track = CreateOrAddOverrideTrack<TrackType>(InPropertyName);
	check(Track);
		
	const FString PropertyPath = InPropertyName.ToString();

	// Split the property path to capture the leaf property name and parent struct to conform
	// with Sequencer Editor property name/path and display name conventions:
	//
	// PropertyName = MyProperty
	// PropertyPath = MyPropertyStruct.MyProperty
	// DisplayName = PropertyName (PropertyStruct)
	FName PropertyName;
	FName PropertyParent;
	int32 NamePos = INDEX_NONE;
	if (PropertyPath.FindLastChar('.', NamePos) && NamePos < PropertyPath.Len() - 1)
	{
		PropertyName = FName(FStringView(*PropertyPath + NamePos + 1, PropertyPath.Len() - NamePos - 1));
		PropertyParent = FName(FStringView(*PropertyPath, NamePos));
	}
	else
	{
		PropertyName = *PropertyPath;
	}
		
	Track->SetPropertyNameAndPath(PropertyName, PropertyPath);

#if WITH_EDITOR
	if (NamePos != INDEX_NONE)
	{
		FText DisplayText = FText::Format(LOCTEXT("DaySequenceActorPropertyTrackFormat", "{0} ({1})"), FText::FromName(PropertyName), FText::FromName(PropertyParent));
		Track->SetDisplayName(DisplayText);
	}
#endif
	return Track;
}

template<typename TrackType, typename SectionType>
SectionType* UProceduralDaySequenceBuilder::CreateOrAddPropertyOverrideSection(FName PropertyName)
{
	TrackType* Track = CreateOrAddPropertyOverrideTrack<TrackType>(PropertyName);
	check(Track);
	return Cast<SectionType>(Track->GetAllSections()[0]);
}

#undef LOCTEXT_NAMESPACE
