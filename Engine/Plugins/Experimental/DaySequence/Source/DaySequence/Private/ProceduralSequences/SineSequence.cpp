// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralSequences/SineSequence.h"

#include "DaySequence.h"
#include "DaySequenceActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SineSequence)

void FSineSequence::BuildSequence(UProceduralDaySequenceBuilder* InBuilder)
{
	using namespace UE::DaySequence;

	ADaySequenceActor* TargetActor = WeakTargetActor.Get();
	
	if (!TargetActor || PropertyName.IsNone())
	{
		return;
	}

	UObject* AnimatedObject = TargetActor;

	if (!ComponentName.IsNone())
	{
		AnimatedObject = GetComponentByName<USceneComponent>(TargetActor, ComponentName);
	}
	
	if (AnimatedObject)
	{
		InBuilder->SetActiveBoundObject(AnimatedObject);

		const double NormalizedTimeIncrement = 1.0 / FMath::Max(KeyCount - 1, static_cast<unsigned>(1));
		for (unsigned int Key = 0; Key < KeyCount; ++Key)
		{
			const double KeyTime = Key * NormalizedTimeIncrement;

			InBuilder->AddScalarKey(PropertyName, KeyTime, Amplitude * FMath::Sin(UE_DOUBLE_TWO_PI * Frequency * (KeyTime - PhaseShift)) + VerticalShift);
		}
	}
}