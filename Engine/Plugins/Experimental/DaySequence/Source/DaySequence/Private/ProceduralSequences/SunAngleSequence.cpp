// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralSequences/SunAngleSequence.h"

#include "DaySequence.h"
#include "DaySequenceActor.h"

#include "Components/DirectionalLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SunAngleSequence)

void FSunAngleSequence::BuildSequence(UProceduralDaySequenceBuilder* InBuilder)
{
	using namespace UE::DaySequence;

	ADaySequenceActor* TargetActor = WeakTargetActor.Get();
	
	if (!TargetActor)
	{
		return;
	}
	
	if (UDirectionalLightComponent* SunComponent = GetComponentByName<UDirectionalLightComponent>(TargetActor, SunComponentName))
	{
		InBuilder->SetActiveBoundObject(SunComponent);

		InBuilder->AddRotationKey(0.0, FRotator(90.0, 0.0, 0.0), RCIM_Linear);
		InBuilder->AddRotationKey(1.0, FRotator(450.0, 0.0, 0.0), RCIM_Linear);
	}
}