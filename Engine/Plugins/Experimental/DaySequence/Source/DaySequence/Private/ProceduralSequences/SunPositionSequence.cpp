// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProceduralSequences/SunPositionSequence.h"

#include "DaySequence.h"
#include "DaySequenceActor.h"

#include "Components/DirectionalLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SunPositionSequence)

namespace UE::DaySequence
{
	struct FSunPositionData
    {
    	/** Sun Elevation */
    	double Elevation;

    	/** Sun Elevation, corrected for atmospheric diffraction */
    	double CorrectedElevation;

    	/** Sun azimuth */
    	double Azimuth;

    	/** Sunrise time */
    	FTimespan SunriseTime;

    	/** Sunset time */
    	FTimespan SunsetTime;

    	/** Solar noon */
    	FTimespan SolarNoon;
    };
	
	void GetSunPosition(double Latitude, double Longitude, double TimeZone, bool bIsDaylightSavingTime, int32 Year, int32 Month, int32 Day, int32 Hours, int32 Minutes, int32 Seconds, int32 Milliseconds, FSunPositionData& SunPositionData)
	{
		if (!FDateTime::Validate(Year, Month, Day, Hours, Minutes, Seconds, Milliseconds))
		{
			return;
		}

		FDateTime CalcTime(Year, Month, Day, Hours, Minutes, Seconds, Milliseconds);

		double TimeOffset = TimeZone;
		if (bIsDaylightSavingTime)
		{
			TimeOffset += 1.0;
		}

		double LatitudeRad = FMath::DegreesToRadians(Latitude);

		// Get the julian day (number of days since Jan 1st of the year 4713 BC)
		double JulianDay = CalcTime.GetJulianDay();
		double JulianCentury = (JulianDay - 2451545.0) / 36525.0;

		// Get the sun's mean longitude , referred to the mean equinox of julian date
		double GeomMeanLongSunDeg = FMath::Fmod(280.46646 + JulianCentury*(36000.76983 + JulianCentury*0.0003032), 360.0);
		double GeomMeanLongSunRad = FMath::DegreesToRadians(GeomMeanLongSunDeg);

		// Get the sun's mean anomaly
		double GeomMeanAnomSunDeg = 357.52911 + JulianCentury*(35999.05029 - 0.0001537*JulianCentury);
		double GeomMeanAnomSunRad = FMath::DegreesToRadians(GeomMeanAnomSunDeg);

		// Get the earth's orbit eccentricity
		double EccentEarthOrbit = 0.016708634 - JulianCentury*(0.000042037 + 0.0000001267*JulianCentury);
		
		// Get the sun's equation of the center
		double SunEqOfCtr = FMath::Sin(GeomMeanAnomSunRad)*(1.914602 - JulianCentury*(0.004817 + 0.000014*JulianCentury))
						  + FMath::Sin(2.0 * GeomMeanAnomSunRad)*(0.019993 - 0.000101*JulianCentury)
						  + FMath::Sin(3.0 * GeomMeanAnomSunRad)*0.000289;
		
		// Get the sun's true longitude
		double SunTrueLongDeg = GeomMeanLongSunDeg + SunEqOfCtr;

		// Get the sun's true anomaly
		//	double SunTrueAnomDeg = GeomMeanAnomSunDeg + SunEqOfCtr;
		//	double SunTrueAnomRad = FMath::DegreesToRadians(SunTrueAnomDeg);

		// Get the earth's distance from the sun
		//	double SunRadVectorAUs = (1.000001018*(1.0 - EccentEarthOrbit*EccentEarthOrbit)) / (1.0 + EccentEarthOrbit*FMath::Cos(SunTrueAnomRad));
		
		// Get the sun's apparent longitude
		double SunAppLongDeg = SunTrueLongDeg - 0.00569 - 0.00478*FMath::Sin(FMath::DegreesToRadians(125.04 - 1934.136*JulianCentury));
		double SunAppLongRad = FMath::DegreesToRadians(SunAppLongDeg);

		// Get the earth's mean obliquity of the ecliptic
		double MeanObliqEclipticDeg = 23.0 + (26.0 + ((21.448 - JulianCentury*(46.815 + JulianCentury*(0.00059 - JulianCentury*0.001813)))) / 60.0) / 60.0;
		
		// Get the oblique correction
		double ObliqCorrDeg = MeanObliqEclipticDeg + 0.00256*FMath::Cos(FMath::DegreesToRadians(125.04 - 1934.136*JulianCentury));
		double ObliqCorrRad = FMath::DegreesToRadians(ObliqCorrDeg);

		// Get the sun's right ascension
		double SunRtAscenRad = FMath::Atan2(FMath::Cos(ObliqCorrRad)*FMath::Sin(SunAppLongRad), FMath::Cos(SunAppLongRad));
		double SunRtAscenDeg = FMath::RadiansToDegrees(SunRtAscenRad);
		
		// Get the sun's declination
		double SunDeclinRad = FMath::Asin(FMath::Sin(ObliqCorrRad)*FMath::Sin(SunAppLongRad));
		double SunDeclinDeg = FMath::RadiansToDegrees(SunDeclinRad);

		double VarY = FMath::Pow(FMath::Tan(ObliqCorrRad / 2.0), 2.0);
		
		// Get the equation of time
		double EqOfTimeMinutes = 4.0 * FMath::RadiansToDegrees(VarY*FMath::Sin(2.0 * GeomMeanLongSunRad) - 2.0 * EccentEarthOrbit*FMath::Sin(GeomMeanAnomSunRad) + 4.0 * EccentEarthOrbit*VarY*FMath::Sin(GeomMeanAnomSunRad)*FMath::Cos(2.0 * GeomMeanLongSunRad) - 0.5*VarY*VarY*FMath::Sin(4.0 * GeomMeanLongSunRad) - 1.25*EccentEarthOrbit*EccentEarthOrbit*FMath::Sin(2.0 * GeomMeanAnomSunRad));

		// Get the hour angle of the sunrise
		double HASunriseDeg = FMath::RadiansToDegrees(FMath::Acos(FMath::Cos(FMath::DegreesToRadians(90.833)) / (FMath::Cos(LatitudeRad)*FMath::Cos(SunDeclinRad)) - FMath::Tan(LatitudeRad)*FMath::Tan(SunDeclinRad)));
		//	double SunlightDurationMinutes = 8.0 * HASunriseDeg;

		// Get the local time of the sun's rise and set
		double SolarNoonLST = (720.0 - 4.0 * Longitude - EqOfTimeMinutes + TimeOffset * 60.0) / 1440.0;
		double SunriseTimeLST = SolarNoonLST - HASunriseDeg * 4.0 / 1440.0;
		double SunsetTimeLST = SolarNoonLST + HASunriseDeg * 4.0 / 1440.0;

		// Get the true solar time
		double TrueSolarTimeMinutes = FMath::Fmod(CalcTime.GetTimeOfDay().GetTotalMinutes() + EqOfTimeMinutes + 4.0 * Longitude - 60.0 * TimeOffset, 1440.0);

		// Get the hour angle of current time
		double HourAngleDeg = TrueSolarTimeMinutes < 0 ? TrueSolarTimeMinutes / 4.0 + 180 : TrueSolarTimeMinutes / 4.0 - 180.0;
		double HourAngleRad = FMath::DegreesToRadians(HourAngleDeg);

		// Get the solar zenith angle
		double SolarZenithAngleRad = FMath::Acos(FMath::Sin(LatitudeRad)*FMath::Sin(SunDeclinRad) + FMath::Cos(LatitudeRad)*FMath::Cos(SunDeclinRad)*FMath::Cos(HourAngleRad));
		double SolarZenithAngleDeg = FMath::RadiansToDegrees(SolarZenithAngleRad);

		// Get the sun elevation
		double SolarElevationAngleDeg = 90.0 - SolarZenithAngleDeg;
		double SolarElevationAngleRad = FMath::DegreesToRadians(SolarElevationAngleDeg);
		double TanOfSolarElevationAngle = FMath::Tan(SolarElevationAngleRad);

		// Get the approximated atmospheric refraction
		double ApproxAtmosphericRefractionDeg = 0.0;
		if (SolarElevationAngleDeg <= 85.0)
		{
			if (SolarElevationAngleDeg > 5.0)
			{
				ApproxAtmosphericRefractionDeg = 58.1 / TanOfSolarElevationAngle - 0.07 / FMath::Pow(TanOfSolarElevationAngle, 3) + 0.000086 / FMath::Pow(TanOfSolarElevationAngle, 5) / 3600.0;
			}
			else
			{
				if (SolarElevationAngleDeg > -0.575)
				{
					ApproxAtmosphericRefractionDeg = 1735.0 + SolarElevationAngleDeg * (-518.2 + SolarElevationAngleDeg * (103.4 + SolarElevationAngleDeg * (-12.79 + SolarElevationAngleDeg * 0.711)));
				}
				else
				{
					ApproxAtmosphericRefractionDeg = -20.772 / TanOfSolarElevationAngle;
				}
			}
			ApproxAtmosphericRefractionDeg /= 3600.0;
		}

		// Get the corrected solar elevation
		double SolarElevationcorrectedforatmrefractionDeg = SolarElevationAngleDeg + ApproxAtmosphericRefractionDeg;

		// Get the solar azimuth 
		double tmp = FMath::RadiansToDegrees(FMath::Acos(((FMath::Sin(LatitudeRad)*FMath::Cos(SolarZenithAngleRad)) - FMath::Sin(SunDeclinRad)) / (FMath::Cos(LatitudeRad)*FMath::Sin(SolarZenithAngleRad))));
		double SolarAzimuthAngleDegcwfromN = HourAngleDeg > 0.0 ? FMath::Fmod(tmp + 180.0, 360.0) : FMath::Fmod(540.0 - tmp, 360.0);


		// offset elevation angle to fit with UE coords system
		SunPositionData.Elevation = 180.0 + SolarElevationAngleDeg;
		SunPositionData.CorrectedElevation = 180.0 + SolarElevationcorrectedforatmrefractionDeg;
		SunPositionData.Azimuth = SolarAzimuthAngleDegcwfromN;
		SunPositionData.SolarNoon = FTimespan::FromDays(SolarNoonLST);
		SunPositionData.SunriseTime = FTimespan::FromDays(SunriseTimeLST);
		SunPositionData.SunsetTime = FTimespan::FromDays(SunsetTimeLST);
	}
}

void FSunPositionSequence::BuildSequence(UProceduralDaySequenceBuilder* InBuilder)
{
	using namespace UE::DaySequence;

	ADaySequenceActor* TargetActor = WeakTargetActor.Get();
	
	if (!TargetActor)
	{
		return;
	}

	if (UDirectionalLightComponent* SunComponent = GetComponentByName<UDirectionalLightComponent>(TargetActor, SunComponentName))
	{
		InBuilder->SetActiveBoundObject(SunComponent);
		
		const double NormalizedTimeIncrement = 1.0 / FMath::Max(KeyCount - 1, static_cast<unsigned>(1));
		for (unsigned int Key = 0; Key < KeyCount; ++Key)
		{
			const double KeyTime = Key * NormalizedTimeIncrement;
		
			FSunPositionData Data;

			double Hours = (double)Key / (double)KeyCount * 24.0;
			int32 IntHours = Hours;

			double Minutes = FMath::Frac(Hours) * 60.0;
			int32 IntMinutes = Minutes;

			double Seconds = FMath::Frac(Minutes) * 60.0;
			int32 IntSeconds = Seconds;

			double Milliseconds = FMath::Frac(Seconds) * 1000.0;
			int32 IntMilliseconds = Milliseconds;
		
			GetSunPosition(Latitude, Longitude, TimeZone, bIsDaylightSavings, Time.GetYear(), Time.GetMonth(), Time.GetDay(), IntHours, IntMinutes, IntSeconds, IntMilliseconds, Data);

			InBuilder->AddRotationKey(KeyTime, FRotator(Data.CorrectedElevation, Data.Azimuth, 0.0));
		}
	}
}