// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DaySequenceActor.h"

#include "BaseDaySequenceActor.generated.h"

#define UE_API DAYSEQUENCE_API

class USkyAtmosphereComponent;
class USkyLightComponent;
class UVolumetricCloudComponent;
class UDirectionalLightComponent;
class UExponentialHeightFogComponent;
class UStaticMeshComponent;

/**
 * A self-registering Day Sequence Actor with a simple set of lighting components (some of which are optional).
 * Can be used as-is or extended by derived classes (see ASunPositionDaySequenceActor).
 */
UCLASS(MinimalAPI, Blueprintable, HideCategories=(Tags, Networking, LevelInstance))
class ABaseDaySequenceActor
	: public ADaySequenceActor
{
	GENERATED_BODY()

public:
	UE_API ABaseDaySequenceActor(const FObjectInitializer& Init);

protected:
	
	/** BeginPlay and OnConstruction overrides auto-register this actor with the DaySequenceSubsystem. */
	UE_API virtual void BeginPlay() override;
	UE_API virtual void OnConstruction(const FTransform& Transform) override;

protected:

	/** Standard Components **/
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> SunRootComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDirectionalLightComponent> SunComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USkyAtmosphereComponent> SkyAtmosphereComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USkyLightComponent> SkyLightComponent;
	
	/** Optional Components **/
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UExponentialHeightFogComponent> ExponentialHeightFogComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UVolumetricCloudComponent> VolumetricCloudComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> SkySphereComponent;
};

#undef UE_API
