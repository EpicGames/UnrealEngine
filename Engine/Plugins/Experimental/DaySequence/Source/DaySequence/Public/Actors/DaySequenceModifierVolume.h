// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "DaySequenceModifierVolume.generated.h"

#define UE_API DAYSEQUENCE_API

class ADaySequenceActor;
class APawn;

class UBoxComponent;
class UDaySequenceModifierComponent;
class UPlayer;

UCLASS(MinimalAPI, Blueprintable)
class ADaySequenceModifierVolume : public AActor
{
	GENERATED_BODY()

public:
	UE_API ADaySequenceModifierVolume(const FObjectInitializer& Init);

	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnDaySequenceActorBound(ADaySequenceActor* InActor);

	UFUNCTION(BlueprintImplementableEvent, CallInEditor)
	UE_API void OnPostEnableModifier();
	
protected:
	//~ Begin AActor interface
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
	//~ End AActor interface
	
	/** Primary initialization entry point to clarify initialization control flow and to consolidate editor and runtime initialization. */
	UE_API void Initialize();
	UE_API void Deinitialize();
	
	/** Wrapper for SetupDaySequenceSubsystemCallbacks and BindToDaySequenceActor. */
	UE_API void DaySequenceActorSetup();

	/** Registers a callback that calls BindToDaySequenceActor when the world's current Day Sequence Actor changes. */
	UE_API void SetupDaySequenceSubsystemCallbacks();

	/** Binds all modifier components to the world's current Day Sequence Actor. */
	UE_API void BindToDaySequenceActor();
	
	/** Initializes a modifier component for this player controller. Will create an additional component if necessary. */
	UE_API void CreatePlayer(APlayerController* InPC);
	
protected:
	UPROPERTY(VisibleAnywhere, Category = "Day Sequence", BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDaySequenceModifierComponent> DaySequenceModifier;

	UPROPERTY(VisibleAnywhere, Category = "Day Sequence", BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UBoxComponent> DefaultBox;
	
	UPROPERTY(Transient, DuplicateTransient)
    TObjectPtr<ADaySequenceActor> DaySequenceActor;
	
	UE_DEPRECATED(5.8, "Replaced with CachedPlayer")
	UPROPERTY(Transient, DuplicateTransient, meta = (DeprecatedProperty, DeprecationMessage = "Replaced with CachedPlayer"))
	TObjectPtr<APlayerController> CachedPlayerController;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UPlayer> CachedPlayer;
	
	FDelegateHandle ReplayScrubbedHandle;

private:
	bool IsSplitscreenSupported() const;
	
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Day Sequence", DisplayName = "Enable Experimental Splitscreen Support")
	bool bEnableSplitscreenSupport;

	/**
	 * Transient modifier components associated with additional local players (primarily for split screen support)
	 * Key: Local player
	 * Value: Optional transient modifier component
	 */
	UPROPERTY(Transient, DuplicateTransient)
	TMap<TObjectPtr<UPlayer>, TObjectPtr<UDaySequenceModifierComponent>> AdditionalPlayers;

	FDelegateHandle ActorSpawnedHandle;
};

#undef UE_API
