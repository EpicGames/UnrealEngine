// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseDaySequenceActor.h"

#include "SunMoonDaySequenceActor.generated.h"

#define UE_API DAYSEQUENCE_API

/**
 * A Day Sequence Actor that represents a physically accurate 24 hour day cycle.
 */
UCLASS(MinimalAPI, Blueprintable)
class ASunMoonDaySequenceActor
	: public ABaseDaySequenceActor
{
	GENERATED_BODY()

public:
	UE_API ASunMoonDaySequenceActor(const FObjectInitializer& Init);

protected:
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Day Sequence", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDirectionalLightComponent> MoonComponent;
};

#undef UE_API
