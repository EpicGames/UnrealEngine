// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/Interface_AssetUserData.h"
#include "MovieSceneSequence.h"
#include "DaySequenceBindingReference.h"
#include "DaySequence.generated.h"

#define UE_API DAYSEQUENCE_API

class UAssetUserData;
class UBlueprint;
struct FMovieSceneSequenceID;

UCLASS(MinimalAPI, BlueprintType, HideCategories=(Object))
class UDaySequence
	: public UMovieSceneSequence
	, public IInterface_AssetUserData
{
	GENERATED_BODY()
public:
	/** Pointer to the movie scene that controls this animation. */
	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

public:
	UE_API UDaySequence(const FObjectInitializer&);
	
	/** Initialize this sequence. */
	UE_API virtual void Initialize();
	UE_API virtual void Initialize(EObjectFlags Flags);

	UE_API void AddDefaultBinding(const FGuid& PossessableGuid);

	UE_API void AddSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization);
	UE_API FGuid GetSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization) const;

	// UMovieSceneSequence interface
	UE_API virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	UE_API virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	UE_API virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	UE_API virtual FGuid FindBindingFromObject(UObject* InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const override;
	UE_API virtual void GatherExpiredObjects(const FMovieSceneObjectCache& InObjectCache, TArray<FGuid>& OutInvalidIDs) const override;
	UE_API virtual UMovieScene* GetMovieScene() const override;
	UE_API virtual UObject* GetParentObject(UObject* Object) const override;
	UE_API virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	UE_API virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext) override;
	UE_API virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* InContext) override;
	UE_API virtual bool AllowsSpawnableObjects() const override;
	UE_API virtual bool CanRebindPossessable(const FMovieScenePossessable& InPossessable) const override;
	UE_API virtual UObject* MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName) override;
	UE_API virtual bool CanAnimateObject(UObject& InObject) const override;
	UE_API virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	UE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITOR
	UE_API virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
	UE_API virtual bool IsFilterSupportedImpl(const FString& InFilterName) const override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	DECLARE_DELEGATE_RetVal_OneParam(void, FPostDuplicateEvent, UDaySequence*);
	static UE_API FPostDuplicateEvent PostDuplicateEvent;
#endif

	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

#if WITH_EDITOR


public:

	/**
	 * Assign a new director blueprint to this sequence. The specified blueprint *must* be contained within this object.?	 */
	UE_API void SetDirectorBlueprint(UBlueprint* NewDirectorBlueprint);

	/**
	 * Retrieve the currently assigned director blueprint for this sequence
	 */
	UE_API UBlueprint* GetDirectorBlueprint() const;

	UE_API FString GetDirectorBlueprintName() const;

protected:

	UE_API virtual FGuid CreatePossessable(UObject* ObjectToPossess) override;
	UE_API virtual FGuid CreateSpawnable(UObject* ObjectToSpawn) override;

	UE_API FGuid FindOrAddBinding(UObject* ObjectToPossess);

	/**
	 * Invoked when this sequence's director blueprint has been recompiled
	 */
	UE_API void OnDirectorRecompiled(UBlueprint*);

#endif // WITH_EDITOR

protected:
	/** References to bound objects. */
	UPROPERTY()
	FDaySequenceBindingReferences BindingReferences;

#if WITH_EDITORONLY_DATA
	/** A pointer to the director blueprint that generates this sequence's DirectorClass. */
	UPROPERTY()
	TObjectPtr<UBlueprint> DirectorBlueprint;
#endif

	/**
	 * The class that is used to spawn this sequence's director instance.
	 * Director instances are allocated on-demand one per sequence during evaluation and are used by event tracks for triggering events.
	 */
	UPROPERTY()
	TObjectPtr<UClass> DirectorClass;

protected:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Animation)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;
};

#undef UE_API
