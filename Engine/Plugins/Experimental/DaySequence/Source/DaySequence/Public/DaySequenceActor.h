// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Info.h"
#include "DaySequenceConditionSet.h"
#include "IDaySequencePlayer.h"
#include "IMovieScenePlaybackClient.h"
#include "MovieSceneBindingOwnerInterface.h"

#include "DaySequenceTime.h"
#include "TimerManager.h"

#include "DaySequenceActor.generated.h"

#define UE_API DAYSEQUENCE_API

#ifndef ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	#define ROOT_SEQUENCE_RECONSTRUCTION_ENABLED WITH_EDITOR
#endif

#ifndef DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	#define DAY_SEQUENCE_ENABLE_DRAW_DEBUG !UE_BUILD_SHIPPING
#endif

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	class AHUD;
#endif

namespace EEndPlayReason { enum Type : int; }

class UCurveFloat;
class UDaySequence;
class UDaySequenceCameraModifierManager;
class UDaySequenceCollectionAsset;
class UDaySequenceConditionTag;
class UDaySequencePlayer;
class UDaySequenceTrack;
class UMovieSceneBindingOverrides;
class UMovieSceneSubSection;

class FDebugDisplayInfo;
struct FDaySequenceCollectionEntry;
struct FMovieSceneSequencePlaybackSettings;

namespace UE::DaySequence
{
	struct FStaticTimeContributor;
	struct FStaticTimeManager;

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	// This provides methods for determining if this debug entry should be shown and for getting a pointer to the debug data.
	// Anything (Currently only DaySequenceModifierComponents) can submit one of these entries to a DaySequenceActor.
	// The debug data can be printed in play with the command "showdebug DaySequence" if ShowCondition evaluates to true. 
	struct FDaySequenceDebugEntry
	{
		using FShowDebugDataConditionFunction = TFunction<bool()>;
		using FGetDebugDataFunction = TFunction<TSharedPtr<TMap<FString, FString>>()>;
		using FGetNumericValue = TFunction<int32()>;

		FDaySequenceDebugEntry(FShowDebugDataConditionFunction InShowCondition, FGetDebugDataFunction InGetData, FGetNumericValue InGetNumericValue);
		
		FShowDebugDataConditionFunction ShowCondition;
		FGetDebugDataFunction GetData;
		FGetNumericValue GetNumericValue;
	};
	
	// Alias type which stores an array of weak pointers to debug entries 
	using FDebugEntryArray = TArray<TWeakPtr<FDaySequenceDebugEntry>>;

	// Alias type which defines the signature of the draw function a registered category must be associated with
	using FDebugCategoryDrawFunction = TFunction<void(UCanvas*, TArray<TSharedPtr<TMap<FString, FString>>>&, const FString&)>;
	
	// Alias type which maps a category to its array of debug entries and to a callback which handles drawing the data.
	using FDebugEntryMap = TMap<FName, TPair<FDebugEntryArray, FDebugCategoryDrawFunction>>;
#endif

	DECLARE_MULTICAST_DELEGATE(FOnInvalidateMuteStates);
}

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTimeOfDayPreviewChanged, float, Time);

UCLASS(MinimalAPI, Blueprintable)
class ADaySequenceActor
	: public AInfo
	, public IMovieScenePlaybackClient
	, public IMovieSceneBindingOwnerInterface
{
	GENERATED_BODY()

public:
	UE_API ADaySequenceActor(const FObjectInitializer& Init);

	/** Access this actor's sequence player, or None if it is invalid (not yet initialized or already destroyed) */
	UE_API IDaySequencePlayer* GetSequencePlayer() const;

	/**
	 * Returns true if the given InDaySequence is referenced by any entry in the DaySequences map property.
	 * 
	 * @param InDaySequence the sequence to search for
	 * @return true if InDaySequence exists in the DaySequences map.
	 */
	UFUNCTION(BlueprintCallable, Category="Sequence")
	UE_API bool ContainsDaySequence(const UDaySequence* InDaySequence);

	/** Set whether or not to replicate playback for this actor */
	UFUNCTION(BlueprintSetter)
	UE_API void SetReplicatePlayback(bool ReplicatePlayback);

#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;

	/**
	 * Set the TimeOfDayPreview if it is currently different from the specified time.
	 * @param InHours preview time to set in hours.
	 */
	UE_API void ConditionalSetTimeOfDayPreview(float InHours);

	/**
	 * Native event invoked when the TimeOfDayPreview property is changed.
	 * @param new preview time in hours.
	 */
	DECLARE_EVENT_OneParam(ADaySequenceActor, FOnTimeOfDayPreviewChangedEvent, float)
	FOnTimeOfDayPreviewChangedEvent OnTimeOfDayPreviewChangedEvent;

	/**
	 * Native event invoked when a subsection is removed from the Root Sequence.
	 * Primarily called by modifier components but generally callable by anything that adds a subsection to the root sequence.
	 * @param The subsection being removed.
	 */
	DECLARE_EVENT_OneParam(ADaySequenceActor, FOnSubSectionRemovedEvent, const UMovieSceneSubSection*)
	static UE_API FOnSubSectionRemovedEvent OnSubSectionRemovedEvent;
#endif //WITH_EDITOR

	/** @return the preview time in hours. */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetTimeOfDayPreview() const;

	/**
	 * Set the TimeOfDayPreview and broadcast the event.
	 * @param InHours preview time to set in hours.
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API void SetTimeOfDayPreview(float InHours);

	/**
     * Get bRunDayCycle, a bool that determines whether or not the day cycle can be played back.
     * @return bRunDayCycle
     */
    UE_API bool GetRunDayCycle() const;

    /**
     * Set bRunDayCycle, a bool that determines whether or not the day cycle can be played back.
     * @param bNewRunDayCycle The new value for bRunDayCycle
     */
    UE_API void SetRunDayCycle(bool bNewRunDayCycle);
	
	/**
	 * Get the length of each day in hours.
	 *
	 * @return float, time in hours.
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetDayLength() const;

	/**
	 * Set the duration of the day in game time in hours.
	 *
	 * This is currently intentionally not exposed to Blueprint since
	 * this property can only be realized by rebuilding the root sequence.
	 *
	 * @param InHours day length in hours.
	 */
	UE_API void SetDayLength(float InHours);

	/**
	 * Get the duration of each day cycle in hours (assuming PlayRate is 1.0)
	 *
	 * @return float, time in hours
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetTimePerCycle() const;

	/**
	 * Set the duration of a day cycle in real time in hours.
	 *
	 * This is currently intentionally not exposed to Blueprint since
	 * the effect is achieved by changing PlayRate at runtime.
	 *
	 * @param InHours time per cycle in hours.
	 */
	UE_API void SetTimePerCycle(float InHours);

	/**
	 * Sets the play rate of the day cycle at runtime.
	 * A play rate of 2.0 will result in a day cycle that is half as long as Time Per Cycle.
	 *
	 * @param NewRate The new play rate.
	 */
	UFUNCTION(BlueprintCallable, NetMulticast, Reliable, Category=RuntimeDayCycle)
	UE_API void SetPlayRate(float NewRate);

	/**
	 * Gets the play rate of the day cycle at runtime. Always 1 in editor worlds.
	 *
	 * @return The current play rate.
	 */
	UFUNCTION(BlueprintCallable, Category=RuntimeDayCycle)
	UE_API float GetPlayRate() const;
	
	/**
	 * Set the duration of a day cycle in real time in hours and update
	 * the root sequence in game.
	 *
	 * This method is the network multicast variant of SetTimePerCycle.
	 * In game, this will properly update the root sequence and subsequence
	 * time scales on both the server and the client. These changes must
	 * be invoked on both server and client since only playback position &
	 * status are replicated.
	 *
	 * @param InHours time per cycle in hours.
	 */
	UE_DEPRECATED(5.6, "Multicast_SetTimePerCycle is deprecated, use SetPlayRate instead.")
	UFUNCTION(NetMulticast, Reliable)
	UE_API void Multicast_SetTimePerCycle(float InHours);

	/**
	 * Get the initial time of day in hours.
	 *
	 * @return float, time in hours
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetInitialTimeOfDay() const;

	/**
	 * Set the initial time of day in game time in hours.
	 *
	 * This is currently intentionally not exposed to Blueprint since
	 * this property can only be realized by rebuilding the root sequence.
	 *
	 * @param InHours initial time of day.
	 */
	UE_API void SetInitialTimeOfDay(float InHours);

	/**
	 * Get the current time of day in hours.
	 *
	 * @return float, time in hours
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetTimeOfDay() const;

	/**
	 * Get the current apparent time of day in hours.
	 * This is distinct from the actual time of day in that it is affected by static time and the day interp curve.
	 *
	 * @return float, time in hours
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetApparentTimeOfDay() const;

	/**
	 * Get the current target apparent time of day in hours.
	 * This is distinct from apparent time of day in that it is only capturing the target time of static time contributors.
	 * The target time is determined by the highest priority static time contributor independent of blend weight. Multiple
	 * contributors at the same priority will return the average of their target times.
	 * 
	 * @return float, time in hours
	 */
	UE_API float GetTargetApparentTimeOfDay() const;

	/**
	 * Set the current time of day in hours (server only).
	 *
	 * @param InHours time of day in hours
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API bool SetTimeOfDay(float InHours);

	/** Resume playback of the sequence. */
	UFUNCTION(BlueprintCallable, Category="Playback")
	UE_API void Play();

	/** Pause playback of the sequence. */
	UFUNCTION(BlueprintCallable, Category="Playback")
	UE_API void Pause();

	/** @return true if the sequence is playing. */
	UFUNCTION(BlueprintCallable, Category="Playback")
	UE_API bool IsPlaying() const;

	/** @return true if the sequence is paused. */
	UFUNCTION(BlueprintCallable, Category="Playback")
	UE_API bool IsPaused() const;

	/**
	 * Check whether this day sequence actor has a static (fixed) time-of-day
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API bool HasStaticTimeOfDay() const;

	/**
	 * Get this day sequence actor's static (fixed) time-of-day, or GetTimeOfDay() if it doesn't have one
	 */
	UFUNCTION(BlueprintCallable, Category="TimeOfDay")
	UE_API float GetStaticTimeOfDay() const;

	UE_API void RegisterStaticTimeContributor(const UE::DaySequence::FStaticTimeContributor& NewContributor) const;
	UE_API void UnregisterStaticTimeContributor(const UObject* InUserObject) const;
	
	UE_API void RegisterBindingResolveFunction(FMovieSceneSequenceID SequenceID, FGuid Guid, TFunction<bool(TArray<UObject*, TInlineAllocator<1>>&)> InFunction);
	UE_API void UnregisterBindingResolveFunction(FMovieSceneSequenceID SequenceID, FGuid Guid = FGuid());

	UDaySequenceCameraModifierManager* GetCameraModifierManager() { return CameraModifierManager; }
	
	UE_API UDaySequence* GetRootSequence() const;

	enum class EUpdateRootSequenceMode
	{
		// Compare the current sequence against the new sequence and only add/remove
		// new/old sequences respectively.
		Update       = 0,
		// Reset and rebuild the root sequence from scratch.
		Reinitialize = 1 << 0,
	};
	
	UE_API virtual void UpdateRootSequence(EUpdateRootSequenceMode Mode = EUpdateRootSequenceMode::Update);
#if WITH_EDITOR
	UE_API void UpdateRootSequenceOnTick(EUpdateRootSequenceMode Mode = EUpdateRootSequenceMode::Update);
#endif

	UE_API bool HasValidRootSequence() const;
	UE_API bool RootSequenceHasValidSections() const;


	DECLARE_EVENT(ADaySequenceActor, FOnRootSequenceChanged)
	FOnRootSequenceChanged& GetOnPostRootSequenceChanged() { return OnPostRootSequenceChanged; }
	FOnRootSequenceChanged& GetOnPreRootSequenceChanged() { return OnPreRootSequenceChanged; }

	/**
	 * This delegate is broadcast after the Day Sequence Actor has initialized all of its own sequences to allow
	 * other sequence providers to initialize their own sequences. If external providers were previously initialized,
	 * they should cache their created subsections and then check to see if they exist in this map. If they do, the bool
	 * associated with that subsection in this map can be set to true and it will be preserved. All subsections that map
	 * to false will be deleted from the root sequence shortly after this delegate is broadcast.
	*/
	using FSubSectionPreserveMap = TMap<UMovieSceneSubSection*, bool>;
	DECLARE_EVENT_OneParam(ADaySequenceActor, FOnPostInitializeDaySequences, FSubSectionPreserveMap*)
	FOnPostInitializeDaySequences& GetOnPostInitializeDaySequences() { return OnPostInitializeDaySequences; }

	/**
	 * This delegate is broadcast at a rate matching this actor's tick interval.
	 * It is either broadcast after each sequence player update or by a timer that is configured to run only when the sequence player is paused.
	 * Used to synchronize polling logic that is not owned by this actor when it should occur at an interval specified by this actor.
	 */
	DECLARE_EVENT(ADaySequenceActor, FOnDaySequenceUpdate)
	FOnDaySequenceUpdate& GetOnDaySequenceUpdate() { return OnDaySequenceUpdate; }
	
	UE_API void InvalidateMuteStates() const;
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	const FName ShowDebug_GeneralCategory = "DaySequence";
	const FName ShowDebug_SubSequenceCategory = "DaySequenceSubSequences";
	
	DECLARE_EVENT_OneParam(ADaySequenceActor, FOnDebugLevelChanged, int32)
	FOnDebugLevelChanged& GetOnDebugLevelChanged() { return OnDebugLevelChanged; }
	int32 GetDebugLevel() const { return CachedDebugLevel; }
	
	UE_API bool IsDebugCategoryRegistered(const FName& Category) const;
	UE_API void RegisterDebugCategory(const FName& Category, UE::DaySequence::FDebugCategoryDrawFunction DrawFunction);

	UE_API void RegisterDebugEntry(const TWeakPtr<UE::DaySequence::FDaySequenceDebugEntry>& DebugEntry, const FName& Category);
	UE_API void UnregisterDebugEntry(const TWeakPtr<UE::DaySequence::FDaySequenceDebugEntry>& DebugEntry, const FName& Category);

	// This will be called once per category and is passed the data for all entries associated with this category
	static UE_API void OnShowDebugInfoDrawFunction(UCanvas* Canvas, TArray<TSharedPtr<TMap<FString, FString>>>& Entries, const FString& Category);
#endif
	
	/**
	 * Normalizes a subsection's timescale and frame range to the root sequence such
	 * that the subsection's sequence represents a full day cycle.
	 */
	UE_API void UpdateSubSectionTimeScale(UMovieSceneSubSection* InSubSection) const;
	
	/**
	 * TODO [nickolas.drake]
	 * Write a class that wraps TrackConditionMap (TrackConditionMapProxy?) and make
	 * EvaluateSequenceConditions and BindToConditionCallbacks member functions.
	 */

	/* Returns a TObjectPtr which is valid if ConditionTagClass is a subclass of UDaySequenceConditionTag and nullptr otherwise. */
	UE_API TObjectPtr<UDaySequenceConditionTag> GetOrInstantiateConditionTag(const TSubclassOf<UDaySequenceConditionTag>& ConditionClass);
	
	/* Called to evaluate a particular set of conditions. Instantiates conditions if necessary. TODO: remove bInitialMuteState */
	UE_API bool EvaluateSequenceConditions(bool bInitialMuteState, const FDaySequenceConditionSet::FConditionValueMap& InConditions);

	/* Called to register InFunction as a callback for instances matching InConditions with a lifetime equivalent to LifetimeObject. */
	UE_API void BindToConditionCallbacks(UObject* LifetimeObject, const FDaySequenceConditionSet::FConditionValueMap& InConditions, const TFunction<void(void)>& InFunction);

#if WITH_EDITOR
	UE_API void HandleConditionReinstanced(const FCoreUObjectDelegates::FReplacementObjectMap& OldToNewInstanceMap);
#endif
	
protected:
	//~ Begin UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface
	
	//~ Begin AActor interface
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void RewindForReplay() override;
	UE_API virtual void Destroyed() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif
	
#if WITH_EDITOR
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
#endif
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool ShouldTickIfViewportsOnly() const override;
	//~ End AActor interface

	//~ Begin IMovieScenePlaybackClient interface
	UE_API virtual bool RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	UE_API virtual UObject* GetInstanceData() const override;
	UE_API virtual bool GetIsReplicatedPlayback() const override;
	UE_API virtual void WarpEvaluationRange(FMovieSceneEvaluationRange& InOutRange) const override;
	//~ End IMovieScenePlaybackClient interface

#if WITH_EDITOR
	//~ Begin IMovieSceneBindingOwnerInterface
	UE_API virtual TSharedPtr<FStructOnScope> GetObjectPickerProxy(TSharedPtr<IPropertyHandle> PropertyHandle) override;
	UE_API virtual void UpdateObjectFromProxy(FStructOnScope& Proxy, IPropertyHandle& ObjectPropertyHandle) override;
	UE_API virtual UMovieSceneSequence* RetrieveOwnedSequence() const override;
	//~ End IMovieSceneBindingOwnerInterface
#endif //WITH_EDITOR

	/** Initialize SequencePlayer with transient root sequence */
	UE_API void InitializePlayer();
	UE_API void InitializeRootSequence();
	UE_API void SetRootSequencePlaybackRange();

	// Returns the rate required for the player to have a duration of TimePerCycle
	UE_API float GetBasePlayRate() const;

	// Returns the duration in hours of the player if its rate is 1
	UE_API float GetBaseDuration() const;

	/* Internal getter that validates the player. */
	UE_API UDaySequencePlayer* GetSequencePlayerInternal() const;

#if ROOT_SEQUENCE_RECONSTRUCTION_ENABLED
	/**
	 * Determines if subsection initialization is necessary or not by verifying all sections in SubSections exist in this map.
	 * If they do, we keep all existing subsections. If not, we remove existing subsections and call InitializeDaySequences().
	 * 
	 * @param SectionsToPreserve A map of RootSequence subsections that allows sequence providers to mark if their cached subsections should be kept or removed.
	 * @return Returns true if we need to remove all sections and reinitialize, false if we can keep all existing sections and bypass initialization.
	 */
	UE_API bool MarkDaySequences(FSubSectionPreserveMap* SectionsToPreserve);
#endif
	
	/**
	 * Called by InitializeRootSequence.
	 * By default this will initialize sequences in DaySequenceCollection.
	 * Non-collection sequences should be initialized in an override of this function in order to be properly inspected by the debug overlay.
	 */
	UE_API virtual void InitializeDaySequences();

	UE_API UMovieSceneSubSection* InitializeDaySequence(const FDaySequenceCollectionEntry& SequenceAsset);

	/** Compute PlaybackSettings from day cycle properties */
	UE_API FMovieSceneSequencePlaybackSettings GetPlaybackSettings(const UDaySequence* Sequence) const;

	UE_API void OnSequencePlayerUpdate(const UDaySequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime);
	UE_API virtual void SequencePlayerUpdated(float CurrentTime, float PreviousTime);

	UE_API float GetInterpCurveTimeOfDay(float Time) const;
	
public:
	/**
	 * User-provided interpolation curve that maps day cycle times to desired cycle times (usually from 0-24 hours).
	 * When disabled, the cycle will interpolate linearly.
	 **/
	UPROPERTY(EditAnywhere, replicated, BlueprintReadWrite, Category=RuntimeDayCycle, meta=(EditCondition=bUseInterpCurve, DisplayAfter=bRunDayCycle))
	TObjectPtr<UCurveFloat> DayInterpCurve;

	UE_DEPRECATED(5.6, "DaySequenceCollection is deprecated in favor of the array property. Please use DaySequenceCollections instead.")
	UPROPERTY()
	TObjectPtr<UDaySequenceCollectionAsset> DaySequenceCollection;

	UPROPERTY(EditAnywhere, Category="Sequence")
	TArray<TObjectPtr<UDaySequenceCollectionAsset>> DaySequenceCollections;

	/** User-defined bias to apply to sequences in DaySequenceCollection. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Sequence", DisplayName="Collection Bias", meta=(DisplayAfter="DaySequenceCollection"))
	int32 Bias;
		
	/** Mapping of actors to override the sequence bindings with */
	UPROPERTY(Instanced, BlueprintReadOnly, Category="Sequence")
	TObjectPtr<UMovieSceneBindingOverrides> BindingOverrides;
	
	/** If true, playback of this sequence on the server will be synchronized across other clients */
	UPROPERTY(EditAnywhere, DisplayName="Replicate Playback", BlueprintReadWrite, BlueprintSetter=SetReplicatePlayback, Category=Replication)
	uint8 bReplicatePlayback : 1;

	/** Used to early out from WarpEvaluationRange. Set to true when editing the root sequence of this actor. */
	bool bForceDisableDayInterpCurve = false;

#if WITH_EDITORONLY_DATA
protected:
	/**
	 * Editor only override for testing in PIE.
	 * If true, the actor uses the current preview time as the initial time of day.
	 * Otherwise, the initial time of day is InitialTimeOfDay.
	 */
	UPROPERTY()
	bool bOverrideInitialTimeOfDay;
	
	/**
	 * Editor only override for testing in PIE.
	 * If true, prevents initial playback on BeginPlay.
	 * Otherwise, initial playback is determined by bRunDayCycle.
	 */
	UPROPERTY()
	bool bOverrideRunDayCycle;

public:
	UE_API bool GetOverrideInitialTimeOfDay() const;
	UE_API void SetOverrideInitialTimeOfDay(bool bNewOverrideInitialTimeOfDay);

	// Variant of SetOverrideInitialTimeOfDay that will update TimeOfDayPreview if necessary.
	UE_API void SetOverrideInitialTimeOfDay(bool bNewOverrideInitialTimeOfDay, float OverrideInitialTimeOfDay);

	UE_API bool GetOverrideRunDayCycle() const;
	UE_API void SetOverrideRunDayCycle(bool bNewOverrideRunDayCycle);
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOverrideInitialTimeOfDayChanged, bool, float);
	FOnOverrideInitialTimeOfDayChanged OnOverrideInitialTimeOfDayChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOverrideRunDayCycleChanged, bool);
	FOnOverrideRunDayCycleChanged OnOverrideRunDayCycleChanged;
#endif
	
protected:
	UPROPERTY(Instanced, transient, replicated)
	TObjectPtr<UDaySequencePlayer> SequencePlayer;
	
	UPROPERTY(Transient)
	TObjectPtr<UDaySequence> RootSequence;

	UPROPERTY(EditAnywhere, Category=RuntimeDayCycle)
	float SequenceUpdateInterval;
	
#if WITH_EDITORONLY_DATA
	/**
	 * Sets the time of day to preview in the editor. Does not affect the start time at runtime.
	 * Can be used as the initial time of day in PIE if bOverrideInitialTimeOfDay is true.
	 */
	UPROPERTY(EditAnywhere, Category = Preview, NonTransactional)
	FDaySequenceTime TimeOfDayPreview;
#endif

	/** Whether or not to run a day cycle. If this is unchecked the day cycle will remain fixed at the time specified by the Initial Time setting */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = SetRunDayCycle, Getter = GetRunDayCycle, Category = RuntimeDayCycle)
	bool bRunDayCycle;

	UPROPERTY(EditAnywhere, replicated, BlueprintReadWrite, Category = RuntimeDayCycle, meta=(InlineEditConditionToggle))
	bool bUseInterpCurve;

	/** How long a single day cycle is */
	UPROPERTY(EditAnywhere, Category = RuntimeDayCycle)
	FDaySequenceTime DayLength;

	/** How long does it take for a day cycle to complete in world time. If this is the same value as day duration that means real world time is used */
	UPROPERTY(EditAnywhere, Category = RuntimeDayCycle)
	FDaySequenceTime TimePerCycle;

	/** The initial time that the day cycle will start at */
	UPROPERTY(EditAnywhere, Category = RuntimeDayCycle)
	FDaySequenceTime InitialTimeOfDay;
	
	TSharedPtr<UE::DaySequence::FStaticTimeManager> StaticTimeManager;

	UPROPERTY(Transient)
	TObjectPtr<UDaySequenceCameraModifierManager> CameraModifierManager;
	
#if WITH_EDITORONLY_DATA
	/**
	 * Blueprint exposed delegate invoked when the TimeOfDayPreview property
	 * is changed.
	 * @param new preview time in hours.
	 */
	UPROPERTY(BlueprintAssignable, Transient, meta=(AllowPrivateAccess="true"))
	FOnTimeOfDayPreviewChanged OnTimeOfDayPreviewChanged;
#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
	struct FUpdateRootSequenceState
	{
		bool bUpdate = false;
		EUpdateRootSequenceMode Mode = EUpdateRootSequenceMode::Update;
	};
	/** When true, the root sequence will be updated on next Tick. */
	FUpdateRootSequenceState UpdateRootSequenceOnTickState = FUpdateRootSequenceState{false, EUpdateRootSequenceMode::Update};
#endif

	FOnRootSequenceChanged OnPreRootSequenceChanged;
	FOnRootSequenceChanged OnPostRootSequenceChanged;
	FOnPostInitializeDaySequences OnPostInitializeDaySequences;
	UE::DaySequence::FOnInvalidateMuteStates OnInvalidateMuteStates;

	FOnDaySequenceUpdate OnDaySequenceUpdate;
	FTimerHandle DaySequenceUpdateTimerHandle;

	/** Starts a timer that will broadcast OnDaySequenceUpdate when the sequence player is paused. */
	UFUNCTION()
	UE_API void StartDaySequenceUpdateTimer();

	/** Stops the timer that was started by DaySequenceUpdateTimerHandle. */
	UFUNCTION()
	UE_API void StopDaySequenceUpdateTimer();
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	UE_API void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	// The keys in this map are debug categories. When a debug entry is registered, it is is added
	// to the array corresponding to the category it specified when registering. When OnShowDebugInfo is called on this
	// actor, the debug data is consolidated and passed to the FDebugCategoryDrawFunction associated with the category
	UE::DaySequence::FDebugEntryMap DebugEntries;

	/** Debug entries this actor is responsible for. Populated by InitializeDaySequence and cleaned by InitializeRootSequence. */
	TArray<TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry>> SubSectionDebugEntries;
	
	// Notifies any listeners when the CVar "DaySequence.DebugLevel" changes and provides the new value.
	FOnDebugLevelChanged OnDebugLevelChanged;
	int32 CachedDebugLevel = 0;
#endif

	TArray<UMovieSceneSubSection*> SubSections;
	
	TMap<FMovieSceneSequenceID, TMap<FGuid, TFunction<bool(TArray<UObject*, TInlineAllocator<1>>&)>>> BindingResolveFunctions;
	
	UPROPERTY(Transient)
	TMap<TSubclassOf<UDaySequenceConditionTag>, TObjectPtr<UDaySequenceConditionTag>> TrackConditionMap;
};

ENUM_CLASS_FLAGS(ADaySequenceActor::EUpdateRootSequenceMode);

#undef UE_API
