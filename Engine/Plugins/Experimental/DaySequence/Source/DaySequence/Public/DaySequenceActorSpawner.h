// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneObjectSpawner.h"

#define UE_API DAYSEQUENCE_API

class FDaySequenceActorSpawner : public IMovieSceneObjectSpawner
{
public:
	static UE_API TSharedRef<IMovieSceneObjectSpawner> CreateObjectSpawner();

	// IMovieSceneObjectSpawner interface
	UE_API virtual UClass* GetSupportedTemplateType() const override;
	UE_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void DestroySpawnedObject(UObject& Object) override;
};

#undef UE_API
