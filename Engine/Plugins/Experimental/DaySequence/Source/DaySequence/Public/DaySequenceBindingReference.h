// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "DaySequenceBindingReference.generated.h"

UENUM()
enum class EDaySequenceBindingReferenceSpecialization
{
	None,						// An unspecialized binding. Resolution is determined by ExternalObjectPath and ObjectPath given some context.
	Root,						// An empty binding with this specialization will always resolve to the Root Day Sequence Actor.
	CameraModifier,				// An empty binding with this specialization will attempt to resolve to a camera modifier associated with a modifier's blend target.
};

/**
 * An external reference to a DaySequence object, resolvable through an arbitrary context.
 * 
 * Bindings consist of an optional package name, and the path to the object within that package.
 * Where package name is empty, the reference is a relative path from a specific outer (the context).
 * Currently, the package name should only ever be empty for component references, which must remain relative bindings to work correctly with spawnables and reinstanced actors.
 */
USTRUCT()
struct FDaySequenceBindingReference
{
	GENERATED_BODY();

	/**
	 * Default construction only used for serialization
	 */
	FDaySequenceBindingReference() {}

	/**
	 * Construct a new binding reference from an object, and a given context (expected to be either a UWorld, or an AActor)
	 */
	DAYSEQUENCE_API FDaySequenceBindingReference(UObject* InObject, UObject* InContext);

	/**
	 * Construct a new binding that always resolves to the day sequence actor
	 */
	DAYSEQUENCE_API static FDaySequenceBindingReference DefaultRootBinding();

	/**
	 * Construct a new binding that always resolves based on specialization type. This is highly context specific.
	 */
	DAYSEQUENCE_API static FDaySequenceBindingReference SpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization);
	
	/**
	 * Resolve this reference within the specified context
	 *
	 * @param InContext		The context to resolve the binding within. Either a UWorld, ULevel (when playing in an instanced level) or an AActor where this binding relates to an actor component
	 * @return The object (usually an Actor or an ActorComponent).
	 */
	DAYSEQUENCE_API UObject* Resolve(UObject* InContext) const;

	/**
	 * Check whether this binding reference is equal to the specified object
	 */
	DAYSEQUENCE_API bool operator==(const FDaySequenceBindingReference& Other) const;

	EDaySequenceBindingReferenceSpecialization GetSpecialization() const { return Specialization; }
	
#if WITH_EDITORONLY_DATA

	/**
	 * Perform legacy data fixup as part of PostLoad
	 */
	void PerformLegacyFixup();

#endif

private:
	/** Path to a specific actor/component inside an external package */
	UPROPERTY()
	TSoftObjectPtr<UObject> ExternalObjectPath;

	/** Object path relative to a passed in context object, this is used if ExternalObjectPath is invalid */
	UPROPERTY()
	FString ObjectPath;

	/** Used when object path data is empty. Generally used for context-specific dynamic bindings. */
	UPROPERTY()
	EDaySequenceBindingReferenceSpecialization Specialization = EDaySequenceBindingReferenceSpecialization::None;
	
#if WITH_EDITORONLY_DATA
	/** The class of the object path. */
	UPROPERTY()
	TSoftClassPtr<UObject> ObjectClass_DEPRECATED;
#endif
};

/**
 * An array of binding references
 */
USTRUCT()
struct FDaySequenceBindingReferenceArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FDaySequenceBindingReference> References;
};


/**
 * Structure that stores a one to many mapping from object binding ID, to object references that pertain to that ID.
 */
USTRUCT()
struct FDaySequenceBindingReferences
{
	GENERATED_BODY()

	/**
	 * Check whether this map has a binding for the specified object id
	 * @return true if this map contains a binding for the id, false otherwise
	 */
	bool HasBinding(const FGuid& ObjectId) const;

	/**
	 * Remove a binding for the specified ID
	 *
	 * @param ObjectId	The ID to remove
	 */
	void RemoveBinding(const FGuid& ObjectId);

	/**
	 * Remove specific object references
	 *
	 * @param ObjectId	The ID to remove
	 * @param InObjects The objects to remove
	 * @param InContext A context in which InObject resides (either a UWorld, or an AActor)
	 */
	void RemoveObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject *InContext);

	/**
	 * Remove specific object references that do not resolve
	 *
	 * @param ObjectId	The ID to remove
	 * @param InContext A context in which InObject resides (either a UWorld, or an AActor)
	 */
	void RemoveInvalidObjects(const FGuid& ObjectId, UObject *InContext);

	/**
	 * Add a binding for the specified ID
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param InObject	The object to associate
	 * @param InContext	A context in which InObject resides (either a UWorld, or an AActor)
	 */
	void AddBinding(const FGuid& ObjectId, UObject* InObject, UObject* InContext);

	/**
	 * Adds a default binding that always resolves to the day sequence actor.
	 * Prefer using AddSpecializedBinding with the Root specialization.
	 *
	 * @param ObjectId	The ID to associate the object with
	 */
	void AddDefaultBinding(const FGuid& ObjectId);

	/**
	 * Adds a specialized binding. 
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param Specialization The type of specialized binding to create
	 */
	void AddSpecializedBinding(const FGuid& ObjectId, EDaySequenceBindingReferenceSpecialization Specialization);

	/**
	 * Finds a specialized binding. 
	 *
	 * @param Specialization The type of specialized binding to find.
	 * @return Specialization The Guid of the found specialized binding, or a default constructed FGuid if not found.
	 */
	
	FGuid FindSpecializedBinding(EDaySequenceBindingReferenceSpecialization Specialization) const;
	
	/**
	 * Resolve a binding for the specified ID using a given context
	 *
	 * @param ObjectId					The ID to associate the object with
	 * @param InContext					A context in which InObject resides
	 * @param OutObjects				Array to populate with resolved object bindings
	 */
	void ResolveBinding(const FGuid& ObjectId, UObject* InContext, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Const accessor for the currently bound anim instance IDs
	 */
	const TSet<FGuid>& GetBoundAnimInstances() const
	{
		return AnimSequenceInstances;
	}

	/**
	 * Filter out any bindings that do not match the specified set of GUIDs
	 *
	 * @param ValidBindingIDs A set of GUIDs that are considered valid. Anything references not matching these will be removed.
	 */
	void RemoveInvalidBindings(const TSet<FGuid>& ValidBindingIDs);

#if WITH_EDITORONLY_DATA

	/**
	 * Perform legacy data fixup as part of PostLoad
	 */
	void PerformLegacyFixup();

#endif

private:

	/** The map from object binding ID to an array of references that pertain to that ID */
	UPROPERTY()
	TMap<FGuid, FDaySequenceBindingReferenceArray> BindingIdToReferences;

	/** A set of object binding IDs that relate to anim sequence instances (must be a child of USkeletalMeshComponent) */
	UPROPERTY()
	TSet<FGuid> AnimSequenceInstances;

	UPROPERTY()
	TMap<EDaySequenceBindingReferenceSpecialization, FGuid> SpecializedReferenceToGuid;
	UPROPERTY()
	TMap<FGuid, EDaySequenceBindingReferenceSpecialization> GuidToSpecializedReference;
};
