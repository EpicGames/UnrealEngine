// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Camera/CameraModifier.h"

#include "DaySequenceCameraModifier.generated.h"

class APlayerController;

class UPostProcessComponent;

/**
 * Provides:
 * 1) an interface for getting UDaySequenceCameraModifiers associated with player controllers,
 * 2) an editor only camera modifier for resolving camera modifier bindings in editor worlds,
 * 3) an editor only post process component for visualizing the editor only camera modifier.
 */
UCLASS(MinimalAPI)
class UDaySequenceCameraModifierManager : public UObject
{
	GENERATED_BODY()
	
public:
	
	UCameraModifier* GetCameraModifier(APlayerController* InPC);
	
#if WITH_EDITOR
	UCameraModifier* GetEditorCameraModifier();
	void UpdateEditorPreview() const;
	void ResetEditorPreview() { EditorCameraModiferPreview = nullptr; }
#endif

private:
	
#if WITH_EDITORONLY_DATA
	/** A camera modifier not associated with any player controller, used to resolve the camera modifier binding in editor. */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UDaySequenceCameraModifier> EditorCameraModifier;
	
	/** A post process component used to preview the animation of EditorCameraModifier. */
    UPROPERTY(Transient, DuplicateTransient)
    TObjectPtr<UPostProcessComponent> EditorCameraModiferPreview;
#endif

	// Note: We have no ownership over either the controller or the modifier. The modifiers are owned by the player camera manager.
	TMap<TWeakObjectPtr<APlayerController>, TWeakObjectPtr<UCameraModifier>> CameraModifiers;

};

UCLASS(MinimalAPI)
class UDaySequenceCameraModifier : public UCameraModifier
{
	GENERATED_BODY()

public:

	const FPostProcessSettings& GetSettings() const { return Settings; }

private:
	
	/**
	 * Post process settings to use for this modifier.
	 * 
	 * Sequencer should handle this (blending multiple sequences, restoring to default, etc),
	 * we just handle forwarding it to the provided settings in ModifyPostProcess()
	 */
	UPROPERTY(EditAnywhere, interp, Category = PostProcess, meta = (ShowOnlyInnerProperties))
	FPostProcessSettings Settings;
	
	virtual void ModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings) override
	{
		PostProcessBlendWeight = 1.f;
		PostProcessSettings = Settings;
	}
};
