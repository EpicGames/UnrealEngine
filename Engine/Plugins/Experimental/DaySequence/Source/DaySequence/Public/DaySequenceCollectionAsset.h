// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequenceConditionSet.h"
#include "Engine/DataAsset.h"
#include "ProceduralDaySequence.h"
#include "StructUtils/InstancedStruct.h"

#include "DaySequenceCollectionAsset.generated.h"

class UDaySequence;

USTRUCT()
struct FDaySequenceCollectionEntry
{
	GENERATED_BODY()
	
	FDaySequenceCollectionEntry(UDaySequence* InDaySequence = nullptr)
	: Sequence(InDaySequence)
	, BiasOffset(0)
	, Conditions(FDaySequenceConditionSet())
	{}

	FDaySequenceCollectionEntry(TObjectPtr<UDaySequence> InDaySequence)
	: FDaySequenceCollectionEntry(InDaySequence.Get())
	{}

	/* The day sequence asset for this collection entry. */
	UPROPERTY(EditAnywhere, Category="Day Sequence", meta=(AllowedClasses="/Script/DaySequence.DaySequence"))
	TObjectPtr<UDaySequence> Sequence;
	
	/* The offset hierarchical bias assigned to this collection entry. */
	UPROPERTY(EditAnywhere, Category="Day Sequence")
	int BiasOffset = 0;
	
	/* The set of conditions which must evaluate to their expected values for this entry to be active. */
	UPROPERTY(EditAnywhere, Category="Day Sequence")
	FDaySequenceConditionSet Conditions;

	UPROPERTY(EditAnywhere, Category="Day Sequence", AdvancedDisplay)
	bool bEnableHierarchicalBlending = true;
};

UCLASS(MinimalAPI, BlueprintType)
class UDaySequenceCollectionAsset : public UDataAsset
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category="Day Sequence")
	TArray<FDaySequenceCollectionEntry> DaySequences;

	UPROPERTY(EditAnywhere, Category="Day Sequence", meta = (ExcludeBaseStruct))
	TArray<TInstancedStruct<FProceduralDaySequence>> ProceduralDaySequences;

	/**
	 * TODO [nickolas.drake]
	 * Add an API which:
	 * 1) Exposes a function called something like "GenerateTransientSequence()" which bakes all sequences into a single transient sequence.
	 * 2) Exposes a delegate which is broadcast when the transient sequence should be considered invalid (most likely due to scalabilty changes).
	 */
};
