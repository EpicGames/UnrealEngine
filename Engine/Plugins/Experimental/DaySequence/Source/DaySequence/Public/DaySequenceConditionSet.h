// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SubclassOf.h"

#include "DaySequenceConditionSet.generated.h"

class UDaySequenceConditionTag;

USTRUCT()
struct FDaySequenceConditionSet
{
	GENERATED_BODY()
	
	/**
	 * TMap where the key specifies a condition tag and the value specifies the
	 * expected value for that condition when evaluating this condition set.
	 * 
	 * Note: This type needs to be specified explicitly for Conditions because we can't use a typedef for a UPROPERTY.
	 */
	typedef TMap<TSubclassOf<UDaySequenceConditionTag>, bool> FConditionValueMap;
	
	UPROPERTY(EditAnywhere, Category = "Conditions")
	TMap<TSubclassOf<UDaySequenceConditionTag>, bool> Conditions;
};