// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"

#include "DaySequenceConditionTag.generated.h"

#define UE_API DAYSEQUENCE_API

class ADaySequenceActor;

/** Enum specifying when a day sequence condition tag will be initialized. */
UENUM(BlueprintType)
enum class EDaySequenceConditionTagInitializationPhase : uint8
{
	Immediate,
	GameStateReady
};

UCLASS(MinimalAPI, Abstract, Blueprintable, Const)
class UDaySequenceConditionTag
	: public UObject
{
public:
	GENERATED_BODY()

	/**
	 * This needs to be called before this condition is expected to function properly.
	 * We do initialization here because there is some uncertainty about which blueprint
	 * functions/events can be safely called while this object is still being constructed.
	 */
	UE_API void Initialize();
	
	/* Evaluates a preconfigured boolean condition. */
	UFUNCTION(BlueprintNativeEvent, Category = "General")
	UE_API bool Evaluate() const;

	UE_API FString GetConditionName() const;
	UE_API EDaySequenceConditionTagInitializationPhase GetInitializationPhase() const;
	
	DECLARE_MULTICAST_DELEGATE(FOnConditionValueChanged);
	FOnConditionValueChanged& GetOnConditionValueChanged() { return OnConditionValueChanged; }
	
	UE_API virtual UWorld* GetWorld() const override;
	
protected:
	/**
	 * Derived classes should override this function if the condition being evaluated is
	 * associated with external delegates which are broadcast when the condition may change.
	 * The intent is to bind BroadcastOnConditionValueChanged to all relevant external delegates so that we
	 * can propagate those broadcasts to notify users of this condition that the condition needs reevaluating.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "General")
	UE_API void SetupOnConditionValueChanged() const;
	
	/**
	 * Derived classes should call this function to notify listeners that the underlying condition may have changed.
	 * This will only trigger a broadcast if Evaluate() returns a different value than the last invocation of this function.
	 */
	UFUNCTION(BlueprintCallable, Category = "General")
	UE_API void BroadcastOnConditionValueChanged() const;

protected:
	/**
	 * Derived classes should give this a meaningful default value which is displayed
	 * when prompting users with a list of possible conditions to apply to a given sequence.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "General")
	FString ConditionName;

	/**
	 * Determines when SetupOnConditionValueChanged is called.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "General")
	EDaySequenceConditionTagInitializationPhase InitializationPhase = EDaySequenceConditionTagInitializationPhase::Immediate;

private:
	FOnConditionValueChanged OnConditionValueChanged;
	
	/* This is an optional because it is unset until the first time BroadcastOnConditionValueChanged is called. */
	mutable TOptional<bool> bCachedEvalResult;
};

#undef UE_API
