// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "DaySequenceDirector.generated.h"

#define UE_API DAYSEQUENCE_API

class AActor;
class UMovieSceneSequence;
struct FMovieSceneObjectBindingID;
struct FQualifiedFrameTime;

class IMovieScenePlayer;
class UDaySequencePlayer;

UCLASS(MinimalAPI, Blueprintable)
class UDaySequenceDirector : public UObject
{
public:
	GENERATED_BODY()

	/** Called when this director is created */
	UFUNCTION(BlueprintImplementableEvent, Category="Sequencer")
	UE_API void OnCreated();
	
	/**
	 * Get the current time for the outermost (root) sequence
	 * @return The current playback position of the root sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	UE_API FQualifiedFrameTime GetRootSequenceTime() const;

	/**
	 * Get the current time for this director's sub-sequence (or the root sequence, if this is a root sequence director)
	 * @return The current playback position of this director's sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	UE_API FQualifiedFrameTime GetCurrentTime() const;

	/**
	 * Resolve the bindings inside this sub-sequence that relate to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	UE_API TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the first valid binding inside this sub-sequence that relates to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	UE_API UObject* GetBoundObject(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the actor bindings inside this sub-sequence that relate to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	UE_API TArray<AActor*> GetBoundActors(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the first valid Actor binding inside this sub-sequence that relates to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	UE_API AActor* GetBoundActor(FMovieSceneObjectBindingID ObjectBinding);

	/*
	 * Get the current sequence that this director is playing back within 
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	UE_API UMovieSceneSequence* GetSequence();

public:

	UE_API virtual UWorld* GetWorld() const override;

	/** Pointer to the player that's playing back this director's sequence. Only valid in game or in PIE/Simulate. */
	UPROPERTY(BlueprintReadOnly, Category="Cinematics")
	TObjectPtr<UDaySequencePlayer> Player;

	/** The Sequence ID for the sequence this director is playing back within - has to be stored as an int32 so that it is reinstanced correctly*/
	UPROPERTY()
	int32 SubSequenceID;

	/** Native player interface index - stored by index so that it can be reinstanced correctly */
	UPROPERTY()
	int32 MovieScenePlayerIndex;
};

#undef UE_API
