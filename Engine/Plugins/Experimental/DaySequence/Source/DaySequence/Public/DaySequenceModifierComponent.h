// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Tickable.h"
#include "Components/BoxComponent.h"
#include "Containers/Ticker.h"
#include "DaySequenceActor.h"
#include "DaySequenceConditionSet.h"
#include "Generators/MovieSceneEasingFunction.h"

#include "DaySequenceModifierComponent.generated.h"

class UMaterialInterface;
class UShapeComponent;
namespace ECollisionEnabled { enum Type : int; }
namespace EEndPlayReason { enum Type : int; }
struct FComponentReference;

class UDaySequence;
class UDaySequenceCollectionAsset;
class UMovieSceneSubSection;

struct FDaySequenceCollectionEntry;

#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
namespace UE::DaySequence
{
	struct FDaySequenceDebugEntry;
}
#endif

/** Enum specifying how to control a day / night cycle from a modifier */
UENUM(BlueprintType)
enum class EDayNightCycleMode : uint8
{
	/** (default) Make no changes to the day/night cycle time */
	Default,
	/** Force the day/night cycle to be fixed at the specified constant time */
	FixedTime,
	/** Set an initial time for the day/night cycle when the modifier is enabled */
	StartAtSpecifiedTime,
	/** Use a random, fixed time for the day/night cycle */
	RandomFixedTime,
	/** Start the day/night cycle at a random time, and allow it to continue from there */
	RandomStartTime,
	/**
	 * Apply a time warp local to the modifier's sequence evaluation time.
	 * Local time warp will not affect global time.
	 */ 
	LocalFixedTime
};

/** Enum that defines modifier behavior for auto enabling and computing the internal blend weight. */
UENUM(BlueprintType)
enum class EDaySequenceModifierMode : uint8
{
	// Blend weight is always 1.0.
	Global,

	// Blend weight smoothly moves between 0.0 and 1.0 according to how far the blend target is from the volume boundary.
	Volume,

	// Blend weight smoothly moves between 0.0 and 1.0 at a fixed rate according to when the blend target last crossed the volume boundary.
	Time
};

/** Enum specifying how the modifier resolves the user specified blend weight against the internal blend weight. */
UENUM(BlueprintType)
enum class EDaySequenceModifierUserBlendPolicy : uint8
{
	// User specified weights are ignored (i.e. the effective weight is InternallyComputedWeight
	Ignored,

	// (default) The effective weight is FMath::Min(InternallyComputedWeight, UserSpecifiedWeight
	Minimum,
	
	// The effective weight is FMath::Max(InternallyComputedWeight, UserSpecifiedWeight
	Maximum,
	
	// The effective weight is UserSpecifiedWeight
	Override
};

#if WITH_EDITOR
	/**
	 * Editor-only tickable class that allows us to enable trigger volume previews based on
	 * persepective camera position in the level viewport.
	 */
	struct FDaySequenceModifierComponentTickableBase : public FTickableGameObject
	{
		FDaySequenceModifierComponentTickableBase()
			: FTickableGameObject(ETickableTickType::Never)
		{}

		virtual void UpdateEditorPreview(float DeltaTime)
		{}

		void Tick(float DeltaTime) override
		{
			/// Overridden here to work around ambiguous Tick function on USceneComponent
			// Re-trigger the function as a differnt named virtual function
			UpdateEditorPreview(DeltaTime);
		}
	};
#else
	/** Empty in non-editor builds */
	struct FDaySequenceModifierComponentTickableBase {};
#endif

UCLASS()
class UDaySequenceModifierEasingFunction
	: public UObject
	, public IMovieSceneEasingFunction
{
public:
	GENERATED_BODY()
	
	enum class EEasingFunctionType
	{
		EaseIn,
		EaseOut
	};
	
	void Initialize(EEasingFunctionType EasingType);
	
	virtual float Evaluate(float Interp) const override;

private:
	TFunction<float(float)> EvaluateImpl;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPostReinitializeSubSequences);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPostEnableModifier);

UCLASS(MinimalAPI, BlueprintType, Blueprintable, config=Game, HideCategories=(Physics,Navigation,Collision,HLOD,Rendering,Cooking,Mobile,RayTracing,AssetUserData), meta=(BlueprintSpawnableComponent))
class UDaySequenceModifierComponent
	: public USceneComponent
	, public FDaySequenceModifierComponentTickableBase
{
public:
	GENERATED_BODY()

	UDaySequenceModifierComponent(const FObjectInitializer& Init);

#if WITH_EDITOR
	DAYSEQUENCE_API static void SetVolumePreviewLocation(const FVector& Location);
	DAYSEQUENCE_API static void SetIsSimulating(bool bInIsSimulating);
#endif

	/**
	 * Bind this component to the specified day sequence actor.
	 * Will not add our overrides to the sub-sequence until EnableModifier is called.
	 */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void BindToDaySequenceActor(ADaySequenceActor* DaySequenceActor);

	/**
	 * Unbind this component from its day sequence actor if valid.
	 * Will removes the sub-sequence from the root sequence if it's set up.
	 */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void UnbindFromDaySequenceActor();

	/**
	 * Enable this component.
	 */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void EnableComponent();

	/**
	 * Disable this component.
	 * Will remove the sub-sequence from the root sequence if it's set up.
	 */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void DisableComponent();

	/**
	 * Rebuilds this component's subsections in the root sequence from the current UserDaySequence/DaySequenceCollections.
	 * Call after swapping collection assets at runtime.
	 */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void ReinitializeSubSequences();

	/** Sets the user day sequence. This must be a user created asset. */
	DAYSEQUENCE_API void SetUserDaySequence(UDaySequence* InDaySequence);
	DAYSEQUENCE_API UDaySequence* GetUserDaySequence() const;

	/** Sets a user specified transient sequence. */
	DAYSEQUENCE_API void SetTransientSequence(UDaySequence* InDaySequence);
	DAYSEQUENCE_API UDaySequence* GetTransientSequence() const;
	
	/** Sets the Day Night Cycle mode. This will reenable the component. */
	DAYSEQUENCE_API void SetDayNightCycle(EDayNightCycleMode NewMode);
	DAYSEQUENCE_API EDayNightCycleMode GetDayNightCycle() const;

	DAYSEQUENCE_API void SetBias(int32 NewBias);
	DAYSEQUENCE_API int32 GetBias() const;
	
	DAYSEQUENCE_API void SetDayNightCycleTime(float Time);
	DAYSEQUENCE_API float GetDayNightCycleTime() const;
	
	DAYSEQUENCE_API void SetMode(EDaySequenceModifierMode NewMode);
	DAYSEQUENCE_API EDaySequenceModifierMode GetMode() const;

	DAYSEQUENCE_API void SetBlendPolicy(EDaySequenceModifierUserBlendPolicy NewPolicy);
	DAYSEQUENCE_API EDaySequenceModifierUserBlendPolicy GetBlendPolicy() const;

	/** Sets the time to blend towards the target weight. */
	DAYSEQUENCE_API void SetBlendTime(float TimeInSeconds);
	DAYSEQUENCE_API float GetBlendTime() const;

	/** Sets a custom blend weight for volume based blends. Final weight depends on BlendPolicy. */
	DAYSEQUENCE_API void SetUserBlendWeight(float Weight);
	DAYSEQUENCE_API float GetUserBlendWeight() const;

	/** Sets the blend target to use when in Volume mode. */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	DAYSEQUENCE_API void SetBlendTarget(APlayerController* InActor);
	
	/** Get the current blend weight. */
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	float GetBlendWeight() const;

	void EmptyVolumeShapeComponents();
	void AddVolumeShapeComponent(const FComponentReference& InShapeReference);
	DAYSEQUENCE_API TArray<UShapeComponent*> GetVolumeShapeComponents() const;

	FOnPostEnableModifier& GetOnPostEnableModifier();

	/*~ Begin UObject interface */
	virtual void PostLoad() override;
	/*~ End UObject interface */
	
	/*~ Begin UActorComponent interface */
	virtual void BeginPlay() override;
	virtual void EndPlay(EEndPlayReason::Type Reason) override;
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	/*~ End UActorComponent interface */
	
#if WITH_EDITOR
	/*~ Begin FTickableGameObject interface */
	virtual void UpdateEditorPreview(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool IsTickable() const override { return true; }
	/*~ End FTickableGameObject interface */

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	
	/** Enable the modifier by enabling its subsection (creating it if necessary) in the Root Sequence. */
	void EnableModifier();

	/** Disable the modifier by disabling its subsection. */
	void DisableModifier();
	
	bool CanBeEnabled() const;

	void SetInitialTimeOfDay();

	/* Called to properly update the mute states of all managed subsections. */
	void InvalidateMuteStates() const;
	
protected:

	/** Non-serialized target actor we are currently bound to */
	UPROPERTY(Transient, DuplicateTransient, BlueprintReadWrite, Category="Day Sequence")
	TObjectPtr<ADaySequenceActor> TargetActor;

	/** A handle used to force an override of the TargetActor's evaluation interval. */
	TSharedPtr<UE::DaySequence::FOverrideUpdateIntervalHandle> OverrideUpdateIntervalHandle;
	
	/** When set, the shape components will be used for the modifier volume, otherwise the default Box component will be used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Day Sequence", meta=(UseComponentPicker, AllowedClasses="/Script/Engine.ShapeComponent", AllowAnyActor))
	TArray<FComponentReference> VolumeShapeComponents;

	/** The actor to use for distance-based volume blend calculations */
	UPROPERTY(Transient, DuplicateTransient)
	TWeakObjectPtr<APlayerController> WeakBlendTarget;

	/** The user provided Day Sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Day Sequence", meta=(EditCondition="!bUseCollection", EditConditionHides, DisplayAfter="bUseCollection"))
	TObjectPtr<UDaySequence> UserDaySequence;

	/** The user provided Transient Day Sequence. */
    UPROPERTY(Transient, BlueprintReadWrite, Setter, Getter, Category="Day Sequence", meta=(EditCondition="!bUseCollection", EditConditionHides, DisplayAfter="bUseCollection"))
    TObjectPtr<UDaySequence> TransientSequence;
	
	/** The user provided collection. This is an alternative to UserDaySequence. */
	UE_DEPRECATED(5.6, "DaySequenceCollection is deprecated in favor of the array property. Please use DaySequenceCollections instead.")
	UPROPERTY()
	TObjectPtr<UDaySequenceCollectionAsset> DaySequenceCollection;
	
	/** The user provided collection. This is an alternative to UserDaySequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Day Sequence", meta=(EditCondition="bUseCollection", EditConditionHides, DisplayAfter="bUseCollection"))
	TArray<TObjectPtr<UDaySequenceCollectionAsset>> DaySequenceCollections;

	UPROPERTY(Instanced, Transient)
	TObjectPtr<UDaySequenceModifierEasingFunction> EasingFunction;
	
	/** User-defined bias. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Day Sequence", meta=(EditCondition="!bIgnoreBias"))
	int32 Bias;

	/** The time to use for the day/night cycle. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, DisplayName="Time", Category="Time", meta=(DisplayAfter="DayNightCycle", EditCondition="DayNightCycle==EDayNightCycleMode::FixedTime || DayNightCycle==EDayNightCycleMode::StartAtSpecifiedTime || DayNightCycle==EDayNightCycleMode::LocalFixedTime", EditConditionHides))
	float DayNightCycleTime;

	/** Defines the region in which the effective blend weight is in the range (0.0, 1.0) (not inclusive) when Mode == EDaySequenceModifierMode::Volume. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Day Sequence", meta=(DisplayAfter="Mode", EditCondition="Mode==EDaySequenceModifierMode::Volume", EditConditionHides))
	float BlendAmount;

	/** Defines the amount of time (in seconds) that it takes for the blend weight to reach the target weight */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Day Sequence", meta=(DisplayAfter="Mode"))
	float BlendTime;

	/** User specified blend weight. The final blend weight is determined by BlendPolicy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Day Sequence", meta=(DisplayAfter="BlendPolicy", EditCondition="BlendPolicy!=EDaySequenceModifierUserBlendPolicy::Ignored", EditConditionHides))
	float UserBlendWeight;

	/** Changes the way the modifier controls the day/night cycle time when enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, DisplayName="Day/Night Cycle", Category="Time")
	EDayNightCycleMode DayNightCycle;

	/** Determines how the modifier computes InternalBlendWeight. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Day Sequence")
	EDaySequenceModifierMode Mode;

	/** Determines how the modifier uses UserBlendWeight to compute effective blend weight. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Day Sequence")
	EDaySequenceModifierUserBlendPolicy BlendPolicy;

	/** Blueprint exposed delegate invoked after the component's subsequences are reinitialized. */
	UPROPERTY(BlueprintAssignable, Transient, meta=(AllowPrivateAccess="true"))
	FOnPostReinitializeSubSequences OnPostReinitializeSubSequences;

	/** Blueprint exposed delegate invoked after the modifier component is enabled. */
	UPROPERTY(BlueprintAssignable, Transient, meta=(AllowPrivateAccess="true"))
	FOnPostEnableModifier OnPostEnableModifier;
	
	/** When enabled, these overrides will always override all settings regardless of their bias. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Day Sequence", meta=(InlineEditConditionToggle))
	uint8 bIgnoreBias : 1;

	/** Flag used track whether or not this component is enabled or disabled. */
	UPROPERTY(Transient, DuplicateTransient, BlueprintReadOnly, Category="Day Sequence")
	uint8 bIsComponentEnabled : 1;
	
	/** Non-serialized variable for tracking whether our overrides are enabled or not. */
	UPROPERTY(Transient, DuplicateTransient, BlueprintReadOnly, Category="Day Sequence")
	uint8 bIsEnabled : 1;

	/** When enabled, preview this day sequence modifier in the editor. */
	UPROPERTY(Transient, BlueprintReadWrite, EditAnywhere, Category="Day Sequence")
	uint8 bPreview : 1;

	/** If true, hide UserDaySequence and expose DaySequenceCollection. */
	UPROPERTY(EditAnywhere, Category="Day Sequence")
	uint8 bUseCollection : 1;

	/** If true, day sequence evaluation while within the blending region will be smooth. Note: Can be very expensive. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Day Sequence")
	uint8 bSmoothBlending : 1;

	/** If true, day sequence evaluation will be smooth regardless of blend weight. Note: Is always be very expensive! */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Day Sequence", meta=(EditCondition="bSmoothBlending", EditConditionHides))
	uint8 bForceSmoothBlending : 1;

private:
	
	/**
	 * Bound to delegate on the DaySequenceActor that allows all modifiers to do work at appropriate times at the specific actors tick interval.
	 * Effectively a 'tick' function.
	 */
	void DaySequenceUpdate();
	
	/**
	 * Get the blend position (handles preview and game world).
	 * @return Returns true if we have a valid blend position and InPosition was set, false if InPosition was not set.
	 */
	bool GetBlendPosition(FVector& InPosition) const;

	/**
	 * Updates InternalBlendWeight, the update method is determined by Mode.
	 * @return Returns InternalBlendWeight.
	 */
	float UpdateInternalBlendWeight();

	/**
	 * The blend weight computed by the modifier.
	 * When this is non-zero the modifier is automatically enabled.
	 * Used to compute effective blend weight along with BlendPolicy and UserBlendWeight.
	 */
	float InternalBlendWeight;

	/** Used by UpdateInternalBlendWeight to compute time delta between InternalBlendWeight updates. */
	float TimedBlendingLastUpdated;
	/** Used by UpdateInternalBlendWeight to compute the weight delta for timed blends */
	float TimedBlendingStartWeight;
	float TimedBlendingTargetWeight;
	/** Used by UpdateInternalBlendWeight to compute time delta between TimedBlendingTargetWeight updates for global/volume timed blends. */
	float TimedBlendingTargetWeightLastUpdated;
	
	/**
	 * Creates and adds or marks for preserve all subsections that the modifier is responsible for.
	 * Optionally provided a map of all sections that exist in the root sequence to a bool flag used to mark that section as still relevant.
	 */
	void ReinitializeSubSequence(ADaySequenceActor::FSubSectionPreserveMap* SectionsToPreserve);
	UMovieSceneSubSection* InitializeDaySequence(const FDaySequenceCollectionEntry& SequenceAsset);
	void RemoveSubSequenceTrack();

	void UpdateCachedExternalShapes() const;
	mutable TArray<TWeakObjectPtr<UShapeComponent>> CachedExternalShapes;
	mutable bool bCachedExternalShapesInvalid = true;
	
	/*~ Transient state for active gameplay */
	TArray<TWeakObjectPtr<UMovieSceneSubSection>> SubSections;

	UE::DaySequence::FOnInvalidateMuteStates OnInvalidateMuteStates;
	
#if DAY_SEQUENCE_ENABLE_DRAW_DEBUG
	const FName ShowDebug_ModifierCategory = "DaySequenceModifiers";
	
	void OnDebugLevelChanged(int32 InDebugLevel);
	bool ShouldShowDebugInfo() const;
	
	/** Determines whether or not the modifier will show debug info */
	int32 DebugLevel;

	TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry> DebugEntry;
	TArray<TSharedPtr<UE::DaySequence::FDaySequenceDebugEntry>> SubSectionDebugEntries;
#endif
};
