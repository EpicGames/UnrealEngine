// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSpawnRegister.h"
#include "UObject/Class.h"

#define UE_API DAYSEQUENCE_API

class IMovieScenePlayer;
class IMovieSceneObjectSpawner;
class UMovieSceneSpawnableBindingBase;

/** Movie scene spawn register that knows how to handle spawning objects (actors) for a DaySequence  */
class FDaySequenceSpawnRegister : public FMovieSceneSpawnRegister
{
public:
	UE_API FDaySequenceSpawnRegister();

protected:
	/** ~ FMovieSceneSpawnRegister interface */
	UE_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override;

#if WITH_EDITOR
	UE_API virtual bool CanSpawnObject(UClass* InClass) const override;
#endif

protected:
	/** Extension object spawners */
	TArray<TSharedRef<IMovieSceneObjectSpawner>> MovieSceneObjectSpawners;
};

#undef UE_API
