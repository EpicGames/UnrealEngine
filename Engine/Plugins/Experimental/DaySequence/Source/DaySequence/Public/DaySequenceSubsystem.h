// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "DaySequenceSubsystem.generated.h"

#define UE_API DAYSEQUENCE_API

class UCheatManager;
class UDaySequenceCheatManagerExtension;

class ADaySequenceActor;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnDaySequenceActorSet, ADaySequenceActor*, NewActor);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDaySequenceActorSetEvent, ADaySequenceActor*);

UCLASS(MinimalAPI)
class UDaySequenceSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	/** UWorldSubsystem interface */
	UE_API virtual bool DoesSupportWorldType(EWorldType::Type WorldType) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UFUNCTION(BlueprintCallable, Category=DaySequence)
	UE_API ADaySequenceActor* GetDaySequenceActor(bool bFindFallbackOnNull = true) const;

	UFUNCTION(BlueprintCallable, Category=DaySequence)
	UE_API void SetDaySequenceActor(ADaySequenceActor* InActor);

	/** Blueprint exposed delegate that is broadcast when the active DaySequenceActor changes. */
	UPROPERTY(BlueprintAssignable, Category=DaySequence)
	FOnDaySequenceActorSet OnDaySequenceActorSet;

	/** Natively exposed delegate that is broadcast when the active DaySequenceActor changes. */
	FOnDaySequenceActorSetEvent OnDaySequenceActorSetEvent;

	UE_API void OnCheatManagerCreated(UCheatManager* InCheatManager);

private:
	/** Utility function that broadcasts both OnDaySequenceActorSet and OnDaySequenceActorSetEvent. */
	void BroadcastOnDaySequenceActorSet(ADaySequenceActor* InActor) const;
	
	TWeakObjectPtr<ADaySequenceActor> DaySequenceActor;

	TWeakObjectPtr<UDaySequenceCheatManagerExtension> CheatManagerExtension;
};

#undef UE_API
