// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequenceTime.generated.h"

USTRUCT()
struct FDaySequenceTime
{
	GENERATED_BODY()

public:
	static constexpr float SecondsPerHour = 3600.f;
	static constexpr float SecondsPerMinute = 60.f;
	
public:
	FDaySequenceTime(int32 InHours = 0, int32 InMinutes = 0, int32 InSeconds = 0);

	bool operator==(const FDaySequenceTime& RHS) const;
	bool operator!=(const FDaySequenceTime& RHS) const;

	float ToHours() const;
	float ToSeconds() const;
	
	static FDaySequenceTime FromHours(float InHours);
	static FDaySequenceTime FromSeconds(float InSeconds);
	
	FString ToString(bool bForceSignDisplay = false) const
	{
		const bool bHasNegativeComponent = Hours < 0 || Minutes < 0 || Seconds < 0;
		const TCHAR* SignText = bHasNegativeComponent ? TEXT("- ") : bForceSignDisplay ? TEXT("+ ") : TEXT("");

		// Allocating string of length 34 to account for 2 sign chars, 10 chars for each int, and 2 ':' chars
		TStringBuilder<34> Builder;
		Builder.Appendf(TEXT("%s%02d:%02d:%02d"), SignText, FMath::Abs(Hours), FMath::Abs(Minutes), FMath::Abs(Seconds));
	
		return Builder.ToString();
	}

public:
	UPROPERTY()
	int32 Hours;

	UPROPERTY()
	int32 Minutes;

	UPROPERTY()
	int32 Seconds;
};