// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieSceneSubTrack.h"
#include "DaySequenceTrack.generated.h"

UCLASS(MinimalAPI)
class UDaySequenceTrack : public UMovieSceneSubTrack
{
	GENERATED_BODY()
public:
	UDaySequenceTrack(const FObjectInitializer& ObjectInitializer);

	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
	FText DisplayName;
#endif
};
