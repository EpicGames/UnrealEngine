// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneObjectSpawner.h"
#include "Modules/ModuleInterface.h"

/**
 * Implements the DaySequence module.
 */
class IDaySequenceModule : public IModuleInterface
{
public:

	/** Register an object spawner */
	virtual FDelegateHandle RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner InOnCreateMovieSceneObjectSpawner) = 0;

	/** Populate the specified array with new object spawner instances for all registered object spawner types */
	virtual void GenerateObjectSpawners(TArray<TSharedRef<IMovieSceneObjectSpawner>>& OutSpawners) const = 0;

	/** Unregister an object spawner */
	virtual void UnregisterObjectSpawner(FDelegateHandle InHandle) = 0;
};
