// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/QualifiedFrameTime.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API DAYSEQUENCE_API

class UDaySequencePlayer;

namespace UE::DaySequence
{
class FOverrideUpdateIntervalHandle
{
public:
	UE_API FOverrideUpdateIntervalHandle(UDaySequencePlayer* InPlayer);
	UE_API ~FOverrideUpdateIntervalHandle();

	UE_API void StartOverriding();
	UE_API void StopOverriding();

private:
	TWeakObjectPtr<UDaySequencePlayer> WeakPlayer;

	bool bIsOverriding;
};
}

class IDaySequencePlayer
{
public:
	
	virtual void Pause() = 0;

	virtual bool HasEverEvaluated() const = 0;

	virtual FQualifiedFrameTime GetCurrentTime() const = 0;
	
	virtual FQualifiedFrameTime GetDuration() const = 0;

	virtual int32 GetCurrentNumLoops() const = 0;
	
	virtual void SetIgnorePlaybackReplication(bool bState) = 0;

	virtual TSharedPtr<UE::DaySequence::FOverrideUpdateIntervalHandle> GetOverrideUpdateIntervalHandle() = 0;
};

#undef UE_API
