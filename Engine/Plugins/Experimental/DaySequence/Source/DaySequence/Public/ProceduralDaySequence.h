// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequenceConditionSet.h"
#include "ProceduralDaySequenceBuilder.h"

#include "GameFramework/Actor.h"

#include "ProceduralDaySequence.generated.h"

#define UE_API DAYSEQUENCE_API

class ADaySequenceActor;
class UDaySequence;

namespace UE::DaySequence
{
	// Utility function to simplify looking for owned components by type and name.
	template <typename T>
	T* GetComponentByName(AActor* InActor, FName Name)
	{
		for (T* Component : TInlineComponentArray<T*>(InActor))
		{
			if (Component->GetFName() == Name)
			{
				return Component;
			}
		}
		
		return nullptr;
	}
}

/**
 * Base class for procedural sequences.
 * To create a procedural sequence, a subclass of this type should be created that overrides BuildSequence.
 * See FSunPositionSequence, FSunAngleSequence, and FSineSequence for examples.
 */
USTRUCT(meta=(Hidden))
struct FProceduralDaySequence
{
	GENERATED_BODY()
	
	virtual ~FProceduralDaySequence() = default;
	
	UE_API UDaySequence* GetSequence(ADaySequenceActor* InActor);

	UPROPERTY(EditAnywhere, Category="Day Sequence")
	FDaySequenceConditionSet Conditions;
	
protected:
	
	virtual void BuildSequence(UProceduralDaySequenceBuilder* InBuilder) {}
	
	TWeakObjectPtr<ADaySequenceActor> WeakTargetActor = nullptr;
};

#undef UE_API
