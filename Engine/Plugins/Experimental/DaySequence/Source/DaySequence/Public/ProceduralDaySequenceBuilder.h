// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DaySequence.h"
#include "Curves/RealCurve.h"

#include "ProceduralDaySequenceBuilder.generated.h"

#define UE_API DAYSEQUENCE_API

class ADaySequenceActor;

/**
 * A utility class for creating procedural Day Sequences.
 * Before adding any keys, SetActiveBoundObject should be called and provided a Day Sequence Actor or a component owned by a Day Sequence Actor.
 * All time values are currently normalized to the range [0, 1], inclusive on both ends. A time of 1 is handled as a special case and maps to the final frame.
 * This class assumes the target Day Sequence Actor will stay alive and that users will keep the generated sequence alive, it manages no lifetimes.
 *
 * Consider using FProceduralDaySequence instead of using this class directly.
 */
UCLASS(MinimalAPI, BlueprintType)
class UProceduralDaySequenceBuilder
	: public UObject
{
	GENERATED_BODY()
	
public:
	/**
	 * Initialize the procedural sequence and set the TargetActor for this builder.
	 * 
	 * @param InActor The target DaySequenceActor that will be animated by the generated sequence.
	 * @param InitialSequence Optional sequence that this builder can operate on instead of allocating a new sequence.
	 * @param bClearInitialSequence If true, calls ClearKeys().
	 * @return The sequence which will be modified when calling SetActiveBoundObject and the Add*Key(s) functions.
	 */
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API UDaySequence* Initialize(ADaySequenceActor* InActor, UDaySequence* InitialSequence = nullptr, bool bClearInitialSequence = true);

	/** Returns true Initialize has been called with a valid actor. */
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API bool IsInitialized() const;
	
	/** Prepare the builder to begin adding keys animating properties on InObject. */
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void SetActiveBoundObject(UObject* InObject);

	
	/* Key Creation: */

	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddBoolOverride(FName PropertyName, bool Value);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddBoolKey(FName PropertyName, float Key, bool Value);
	UE_API void AddBoolKey(FName PropertyName, const TPair<float, bool>& KeyValue);
	UE_API void AddBoolKeys(FName PropertyName, const TArray<TPair<float, bool>>& KeysAndValues);
	
	
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddScalarOverride(FName PropertyName, double Value);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddScalarKey(FName PropertyName, float Key, double Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UE_API void AddScalarKey(FName PropertyName, const TPair<float, double>& KeyValue, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UE_API void AddScalarKeys(FName PropertyName, const TArray<TPair<float, double>>& KeysAndValues, ERichCurveInterpMode InterpMode = RCIM_Cubic);

	
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddVectorOverride(FName PropertyName, FVector Value);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddVectorKey(FName PropertyName, float Key, FVector Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UE_API void AddVectorKey(FName PropertyName, const TPair<float, FVector>& KeyValue, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UE_API void AddVectorKeys(FName PropertyName, const TArray<TPair<float, FVector>>& KeysAndValues, ERichCurveInterpMode InterpMode = RCIM_Cubic);

	
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddColorOverride(FName PropertyName, FLinearColor Value);
	UE_API void AddColorKeys(FName PropertyName, const TArray<TPair<float, FLinearColor>>& KeysAndValues, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	
	
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddTransformOverride(const FTransform& Value);
	UE_API void AddTransformKey(float Key, const FTransform& Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddTranslationKey(float Key, const FVector& Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddRotationKey(float Key, const FRotator& Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddScaleKey(float Key, const FVector& Value, ERichCurveInterpMode InterpMode = RCIM_Cubic);
	

	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void AddMaterialOverride(int32 MaterialIndex, UMaterialInterface* Value);

	
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	UE_API void AddScalarMaterialParameterOverride(FName ParameterName, int32 MaterialIndex, float Value);
	UE_API void AddScalarMaterialParameterKeys(FName ParameterName, int32 MaterialIndex, const TArray<TPair<float, float>>& KeysAndValues);

	
	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	UE_API void AddColorMaterialParameterOverride(FName ParameterName, int32 MaterialIndex, FLinearColor Value);
	UE_API void AddColorMaterialParameterKeys(FName ParameterName, int32 MaterialIndex, const TArray<TPair<float, FLinearColor>>& KeysAndValues);


	UFUNCTION(BlueprintCallable, Category="Day Sequence")
	UE_API void AddVisibilityOverride(bool bValue);
	UE_API void AddVisibilityKeys(const TArray<TPair<float, bool>>& KeysAndValues);
	
	
	/* Key Deletion: */
	
	UFUNCTION(BlueprintCallable, Category = "Day Sequence")
	UE_API void ClearKeys();
	
private:
	FGuid GetOrCreateProceduralBinding(UObject* Object) const;

	template<typename TrackType>
	TrackType* CreateOrAddOverrideTrack(FName Name);

	template<typename TrackType>
	TrackType* CreateOrAddPropertyOverrideTrack(FName Name);

	template<typename TrackType, typename SectionType>
	SectionType* CreateOrAddPropertyOverrideSection(FName Name);
	
private:
	/** This is returned immediately upon creation in InitializeSequence. The caller is responsible for holding a reference to prevent GC. */
	TObjectPtr<UDaySequence> ProceduralDaySequence = nullptr;

	TObjectPtr<ADaySequenceActor> TargetActor = nullptr;
	
	TObjectPtr<UObject> ActiveBoundObject = nullptr;
	FGuid ActiveBinding;
};

#undef UE_API
