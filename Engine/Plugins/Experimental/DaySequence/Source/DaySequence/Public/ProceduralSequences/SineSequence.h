// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralDaySequence.h"

#include "SineSequence.generated.h"

#define UE_API DAYSEQUENCE_API

/**
 * A procedural sequence that animates a user specified property according to a sine wave.
 */
USTRUCT()
struct FSineSequence : public FProceduralDaySequence
{
	GENERATED_BODY()

	virtual ~FSineSequence() override = default;
	
	// The name of the float property to animate.
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	FName PropertyName = FName(TEXT(""));
	
	// Optional component, used if the property being animated is on a component owned by the Day Sequence Actor.
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	FName ComponentName = FName(TEXT(""));

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	unsigned KeyCount = 5;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	float Amplitude = 1.f;
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	float Frequency = 1.f;
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	float PhaseShift = 0.f;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	float VerticalShift = 0.f;
	
private:
	
	UE_API virtual void BuildSequence(UProceduralDaySequenceBuilder* InBuilder) override;
};

#undef UE_API
