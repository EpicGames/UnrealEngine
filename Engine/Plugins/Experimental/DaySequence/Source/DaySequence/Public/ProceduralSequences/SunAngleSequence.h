// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralDaySequence.h"

#include "SunAngleSequence.generated.h"

#define UE_API DAYSEQUENCE_API

/**
 * A procedural sequence that linearly animates the sun.
 */
USTRUCT()
struct FSunAngleSequence : public FProceduralDaySequence
{
	GENERATED_BODY()

	virtual ~FSunAngleSequence() override = default;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	FName SunComponentName = FName(TEXT("Sun"));

private:
	
	UE_API virtual void BuildSequence(UProceduralDaySequenceBuilder* InBuilder) override;
};

#undef UE_API
