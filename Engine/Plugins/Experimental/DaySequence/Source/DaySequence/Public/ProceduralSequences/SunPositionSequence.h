// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralDaySequence.h"

#include "SunPositionSequence.generated.h"

#define UE_API DAYSEQUENCE_API

/**
 * A procedural sequence that animates a sun in a physically accurate way based on geographic data.
 */
USTRUCT()
struct FSunPositionSequence : public FProceduralDaySequence
{
	GENERATED_BODY()

	virtual ~FSunPositionSequence() override = default;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	FName SunComponentName = FName(TEXT("Sun"));
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	unsigned KeyCount = 24;
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	FDateTime Time;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	double TimeZone = 0.0;
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	double Latitude = 0.0;

	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	double Longitude = 0.0;
	
	UPROPERTY(EditAnywhere, Category = "Procedural Parameters")
	bool bIsDaylightSavings = false;
	
private:
	
	UE_API virtual void BuildSequence(UProceduralDaySequenceBuilder* InBuilder) override;
};

#undef UE_API
