// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DaySequenceEditor : ModuleRules
{
	public DaySequenceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"ClassViewer",
				"Projects",
				"InputCore",
				"EditorFramework",
				"EditorStyle",
				"UnrealEd",
				"EditorWidgets",
				"EditorSubsystem",
				"ToolMenus",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"WorkspaceMenuStructure",
				"SubobjectEditor",
				"DaySequence",
				"ToolWidgets",
				"LevelSequence",
				"LevelSequenceEditor",
				"Sequencer",
				"SequencerScripting",
				"MovieSceneCaptureDialog",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"SceneOutliner",
				"ViewportInteraction",
				"VREditor"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
				"PropertyEditor"
			}
			);
	}
}
