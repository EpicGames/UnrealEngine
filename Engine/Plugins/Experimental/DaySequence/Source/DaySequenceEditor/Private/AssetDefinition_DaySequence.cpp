// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DaySequence.h"
#include "DaySequence.h"
#include "DaySequenceEditorStyle.h"
#include "DaySequenceEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DaySequence)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FAssetOpenSupport UAssetDefinition_DaySequence::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit, EToolkitMode::WorldCentric); 
}

EAssetCommandResult UAssetDefinition_DaySequence::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UWorld* WorldContext = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor)
		{
			WorldContext = Context.World();
			break;
		}
	}

	if (!ensure(WorldContext))
	{
		return EAssetCommandResult::Handled;
	}

	for (UDaySequence* DaySequence : OpenArgs.LoadObjects<UDaySequence>())
	{
		TSharedRef<FDaySequenceEditorToolkit> Toolkit = MakeShareable(new FDaySequenceEditorToolkit(FDaySequenceEditorStyle::Get()));
		Toolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, DaySequence);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
