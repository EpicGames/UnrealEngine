// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequence.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_DaySequence.generated.h"

UCLASS()
class UAssetDefinition_DaySequence : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DaySequence", "Day Sequence"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200, 80, 80)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDaySequence::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
