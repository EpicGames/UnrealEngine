// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DaySequenceCollectionAsset.h"

#include "DaySequenceCollectionAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DaySequenceCollectionAsset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TSoftClassPtr<UObject> UAssetDefinition_DaySequenceCollectionAsset::GetAssetClass() const
{
	return UDaySequenceCollectionAsset::StaticClass();
}

UDaySequenceCollectionAssetFactory::UDaySequenceCollectionAssetFactory(const class FObjectInitializer& Obj)
: Super(Obj)
{
	SupportedClass = UDaySequenceCollectionAsset::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UDaySequenceCollectionAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UDaySequenceCollectionAsset>(InParent, Class, Name, Flags | RF_Transactional, Context);
}

#undef LOCTEXT_NAMESPACE
