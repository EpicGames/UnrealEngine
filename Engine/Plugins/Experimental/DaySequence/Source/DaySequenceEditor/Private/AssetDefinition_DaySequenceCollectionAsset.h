// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Factories/Factory.h"

#include "AssetDefinition_DaySequenceCollectionAsset.generated.h"

#define UE_API DAYSEQUENCEEDITOR_API

class UDaySequenceCollectionAsset;

UCLASS()
class UAssetDefinition_DaySequenceCollectionAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DaySequenceCollectionAsset", "Day Sequence Collection"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200, 80, 80)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
};

UCLASS(MinimalAPI)
class UDaySequenceCollectionAssetFactory : public UFactory
{
	GENERATED_BODY()
	
public:
	UE_API UDaySequenceCollectionAssetFactory(const class FObjectInitializer& Obj);
	
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

#undef UE_API
