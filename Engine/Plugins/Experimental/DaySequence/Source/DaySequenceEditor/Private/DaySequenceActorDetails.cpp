// Copyright Epic Games, Inc. All Rights Reserved.
#include "DaySequenceActorDetails.h"
#include "DaySequenceActor.h"
#include "MovieSceneBindingOverrides.h"
#include "SDaySequencePreviewTimeSlider.h"

#include "Algo/Transform.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "DaySequenceActorDetails"

namespace DaySequenceActorDetailsLocals
{
	void AddAllSubObjectProperties(TArray<UObject*>& SubObjects, IDetailCategoryBuilder& Category, TAttribute<EVisibility> Visibility = TAttribute<EVisibility>(EVisibility::Visible))
	{
		SubObjects.Remove(nullptr);
		if (!SubObjects.Num())
		{
			return;
		}

		for (const FProperty* TestProperty : TFieldRange<FProperty>(SubObjects[0]->GetClass()))
		{
			if (TestProperty->HasAnyPropertyFlags(CPF_Edit))
			{
				const bool bAdvancedDisplay = TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay);
				const EPropertyLocation::Type PropertyLocation = bAdvancedDisplay ? EPropertyLocation::Advanced : EPropertyLocation::Common;

				IDetailPropertyRow* NewRow = Category.AddExternalObjectProperty(SubObjects, TestProperty->GetFName(), PropertyLocation);
				if (NewRow)
				{
					NewRow->Visibility(Visibility);
				}
			}
		}
	}
}

TSharedRef<IDetailCustomization> FDaySequenceActorDetails::MakeInstance()
{
	return MakeShared<FDaySequenceActorDetails>();
}

void FDaySequenceActorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Identify the DaySequenceActors in the selection and set the first occurrence
	// as the primary.
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailLayout.GetSelectedObjects();
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[ObjectIndex];
		if (CurrentObject.IsValid())
		{
			if (ADaySequenceActor* CurrentDaySequenceActor = Cast<ADaySequenceActor>(CurrentObject.Get()))
			{
				DaySequenceActor = CurrentDaySequenceActor;
				break;
			}
		}
	}

	TArray<ADaySequenceActor*> DaySequenceActors;
	{
		TArray<TWeakObjectPtr<>> ObjectPtrs;
		DetailLayout.GetObjectsBeingCustomized(ObjectPtrs);

		for (TWeakObjectPtr<> WeakObj : ObjectPtrs)
		{
			if (ADaySequenceActor* Actor = Cast<ADaySequenceActor>(WeakObj.Get()))
			{
				DaySequenceActors.Add(Actor);
			}
		}
	}
	
	DetailLayout.HideProperty("DefaultComponents");

	/*IDetailCategoryBuilder& SequenceCategory =*/ DetailLayout.EditCategory("Sequence", NSLOCTEXT("DaySequenceActorDetails", "Sequence", "Sequence"), ECategoryPriority::Important).InitiallyCollapsed(false);

	IDetailCategoryBuilder& PreviewCategory = DetailLayout.EditCategory("Preview", NSLOCTEXT("DaySequenceActorDetails", "Preview", "Preview"), ECategoryPriority::Important).InitiallyCollapsed(false);
	PreviewCategory.AddProperty(DetailLayout.GetProperty("TimeOfDayPreview"));
	PreviewCategory.AddCustomRow(NSLOCTEXT("DaySequenceActorDetails", "PreviewSequenceSlider", "Preview Sequence"))
	.RowTag("Preview Sequence Slider")
	[
		SNew(SDaySequencePreviewTimeSlider)
	];

	DetailLayout.EditCategory("RuntimeDayCycle", NSLOCTEXT("TimeOfActorDetails", "RuntimeDayCycle", "Runtime Day Cycle"), ECategoryPriority::Important).InitiallyCollapsed(false);

	TArray<UObject*> SubObjects;

	IDetailCategoryBuilder& BindingOverridesCategory = DetailLayout.EditCategory("BindingOverrides", NSLOCTEXT("TimeOfActorDetails", "BindingOverrides", "Binding Overrides"), ECategoryPriority::Important);
	{
		SubObjects.Reset();
		Algo::Transform(DaySequenceActors, SubObjects, &ADaySequenceActor::BindingOverrides);
		DaySequenceActorDetailsLocals::AddAllSubObjectProperties(SubObjects, BindingOverridesCategory);
	}

	// Hide the Environment category that holds our subobject components since that UI is not useful.
	DetailLayout.HideCategory("Environment");
	
	DetailLayout.HideCategory("Physics");
	DetailLayout.HideCategory("HLOD");
	DetailLayout.HideCategory("Activation");
	DetailLayout.HideCategory("Input");
	DetailLayout.HideCategory("Collision");
	DetailLayout.HideCategory("Actor");
	DetailLayout.HideCategory("Lod");
	DetailLayout.HideCategory("Cooking");
}

#undef LOCTEXT_NAMESPACE
