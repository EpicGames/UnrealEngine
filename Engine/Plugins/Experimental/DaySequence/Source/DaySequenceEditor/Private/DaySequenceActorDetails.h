// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class ADaySequenceActor;
class ISequencer;

class FDaySequenceActorDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
	
private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	
private:
	/** The selected DaySequence sequence actor */
	TWeakObjectPtr<ADaySequenceActor> DaySequenceActor;
};
