// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceActorFactory.h"

#include "AssetRegistry/AssetData.h"
#include "DaySequenceCollectionAsset.h"
#include "DaySequenceActor.h"
#include "DaySequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceActorFactory)

#define LOCTEXT_NAMESPACE "DaySequenceEditor"

UDaySequenceActorFactory::UDaySequenceActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DaySequenceDisplayName", "Day Sequence");
	NewActorClass = ADaySequenceActor::StaticClass();
}

bool UDaySequenceActorFactory::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if (UActorFactory::CanCreateActorFrom(AssetData, OutErrorMsg))
	{
		return true;
	}

	if (AssetData.IsValid() && !AssetData.IsInstanceOf(UDaySequence::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoDaySequenceAsset", "A valid Day sequence asset must be specified.");
		return false;
	}
	
	return true;
}

AActor* UDaySequenceActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	ADaySequenceActor* NewActor = Cast<ADaySequenceActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));
	return NewActor;
}

UObject* UDaySequenceActorFactory::GetAssetFromActorInstance(AActor* Instance)
{
	if (const ADaySequenceActor* DaySequenceActor = Cast<ADaySequenceActor>(Instance))
	{
		if (DaySequenceActor->DaySequenceCollections.Num() > 0)
		{
			return DaySequenceActor->DaySequenceCollections[0];
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE


