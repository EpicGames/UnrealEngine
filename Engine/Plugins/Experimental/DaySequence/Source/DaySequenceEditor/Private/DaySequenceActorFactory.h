// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "DaySequenceActorFactory.generated.h"

struct FActorSpawnParameters;

class AActor;
struct FAssetData;

UCLASS()
class UDaySequenceActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};

