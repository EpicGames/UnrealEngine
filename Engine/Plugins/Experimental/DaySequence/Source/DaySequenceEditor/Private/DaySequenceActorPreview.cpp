// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceActorPreview.h"
#include "DaySequenceActor.h"
#include "DaySequenceEditorStyle.h"
#include "DaySequenceEditorToolkit.h"
#include "DaySequenceSubsystem.h"

#include "Editor.h"
#include "Engine/Blueprint.h"
#include "IDetailsView.h"
#include "ISequencer.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Misc/CoreDelegates.h"
#include "Misc/QualifiedFrameTime.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "PropertyEditorModule.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

FDaySequenceActorPreview::~FDaySequenceActorPreview()
{
}

void FDaySequenceActorPreview::UpdateActorPreview()
{
	// Update our active DaySequence actor from the editor world.
	if (GEditor)
	{
		if (const UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			if (const UDaySequenceSubsystem* DaySubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
			{
				if (ADaySequenceActor* NewDayActor = DaySubsystem->GetDaySequenceActor(/* bFindFallbackOnNull */ false); NewDayActor != DaySequenceActor)
				{
					if (DaySequenceActor.IsValid())
					{
						DaySequenceActor->OnTimeOfDayPreviewChangedEvent.RemoveAll(this);
						DaySequenceActor->GetOnPreRootSequenceChanged().RemoveAll(this);
						DaySequenceActor->GetOnPostRootSequenceChanged().RemoveAll(this);
					}

					if (NewDayActor)
					{
						NewDayActor->OnTimeOfDayPreviewChangedEvent.AddRaw(this, &FDaySequenceActorPreview::OnTimeOfDayPreviewChanged);
						NewDayActor->GetOnPreRootSequenceChanged().AddRaw(this, &FDaySequenceActorPreview::OnPreRootSequenceChanged);
						NewDayActor->GetOnPostRootSequenceChanged().AddRaw(this, &FDaySequenceActorPreview::OnPostRootSequenceChanged);
					}
		
					DaySequenceActor = NewDayActor;

					// Close any active preview if our DaySequenceActor has changed.
					ClosePreviewToolkit();
				}
			}
		}
	}
}

bool FDaySequenceActorPreview::IsValid() const
{
	return DaySequenceActor.IsValid() && DaySequenceActor->RootSequenceHasValidSections();
}

TWeakObjectPtr<ADaySequenceActor> FDaySequenceActorPreview::GetPreviewActor() const
{
	return DaySequenceActor;
}

TWeakPtr<ISequencer> FDaySequenceActorPreview::GetPreviewSequencer() const
{
	return DaySequencePreviewToolkit.IsValid() ? DaySequencePreviewToolkit.Pin()->GetSequencer() : nullptr;
}

float FDaySequenceActorPreview::GetPreviewTime() const
{
	float PreviewTime = 0.0f;
	if (DaySequenceActor.IsValid() && DaySequencePreviewToolkit.IsValid())
	{
		// Convert sequencer time to equivalent game time.
		const TSharedPtr<ISequencer> Sequencer = DaySequencePreviewToolkit.Pin()->GetSequencer();

		const FFrameNumber LowerBound = Sequencer->GetPlaybackRange().GetLowerBoundValue();
		const FFrameNumber UpperBound = Sequencer->GetPlaybackRange().GetUpperBoundValue();
		const int32 Range = UpperBound.Value - LowerBound.Value;
		const int32 CurrentTimeOffset = Sequencer->GetGlobalTime().Time.FrameNumber.Value - LowerBound.Value;
		const float NormalizedTime = static_cast<float>(CurrentTimeOffset) / Range;

		PreviewTime = NormalizedTime * DaySequenceActor->GetDayLength();
	}
	return PreviewTime;
}

void FDaySequenceActorPreview::SetPreviewTime(float NewPreviewTime)
{
	LastPreviewTime = NewPreviewTime;
	
	if (!DaySequenceActor.IsValid() || !DaySequencePreviewToolkit.IsValid())
	{
		return;
	}

	const float CurrentPreviewTime = GetPreviewTime();

	// If this check fails for nearly equal values, we can get into a state of infinite recursion (that seems to only reproduce in shipping builds).
	if (FMath::IsNearlyEqual(NewPreviewTime, CurrentPreviewTime, UE_KINDA_SMALL_NUMBER))
	{
		return;
	}
	
	const TSharedPtr<ISequencer> Sequencer = DaySequencePreviewToolkit.Pin()->GetSequencer();
	
	// Given game time (NewPreviewTime), convert to equivalent sequencer time.
	const float NormalizedTime = NewPreviewTime / DaySequenceActor->GetDayLength();
	
	const FFrameNumber LowerBound = Sequencer->GetPlaybackRange().GetLowerBoundValue();
	const FFrameNumber UpperBound = Sequencer->GetPlaybackRange().GetUpperBoundValue();
	const int32 Range = UpperBound.Value - LowerBound.Value;
	const int32 CurrentTimeOffset = NormalizedTime * Range;
	const FFrameNumber CurrentTime = LowerBound + CurrentTimeOffset;

	const FFrameTime SequencerFrameTime = FMath::Clamp<FFrameTime>(CurrentTime, 0, UpperBound);
	
	if (Sequencer->IsEvaluating())
	{
		constexpr bool bEvaluateImmediately = false;
		Sequencer->SetGlobalTime(SequencerFrameTime, bEvaluateImmediately);
		Sequencer->RequestEvaluate();
	}
	else
	{
		constexpr bool bEvaluateImmediately = true;
		Sequencer->SetGlobalTime(SequencerFrameTime, bEvaluateImmediately);
	}

	if (!GEditor || GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor)
	{
		return;
	}

	// Set the EditingThroughMovementWidget flag on the level viewport clients to
	// trigger more immediate lighting updates.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			LevelVC->SetEditingThroughMovementWidget();
		}
	}
}

bool FDaySequenceActorPreview::IsPreviewEnabled() const
{
	return DaySequencePreviewToolkit.IsValid() && DaySequencePreviewToolkit.Pin()->IsActive();
}

float FDaySequenceActorPreview::GetDayLength() const
{
	return DaySequenceActor.IsValid() ? DaySequenceActor->GetDayLength() : 24.f;
}

void FDaySequenceActorPreview::EnablePreview(bool bEnable)
{
	ClosePreviewToolkit();

	if (DaySequenceActor.IsValid())
	{

		// Toolkits will automatically close other toolkits on initialization. Actor preview
		// is a passive mode that we always want to have active when there is no active
		// Sequence editor toolkit open.
		if (bEnable && IsValid() && !FDaySequenceEditorToolkit::HasOpenSequenceEditorToolkits())
		{
			GEditor->OnBlueprintPreCompile().AddRaw(this, &FDaySequenceActorPreview::OnBlueprintPreCompile);
			GEditor->OnEditorClose().AddRaw(this, &FDaySequenceActorPreview::Deregister);

			if (const UWorld* World = GEditor->GetEditorWorldContext().World())
            {
            	if (UDaySequenceSubsystem* DaySubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
            	{
            		// Using a lambda here because the delegate signature does not match UpdateActorPreview's signature
            		DaySubsystem->OnDaySequenceActorSetEvent.AddLambda([this](ADaySequenceActor* NewActor) { UpdateActorPreview(); });
            	}
            }
			
			const TSharedPtr<IToolkitHost> EditWithinLevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstLevelEditor();
			if (EditWithinLevelEditor.IsValid())
			{
				// InitializeActorPreview will register the toolkit with the toolkit manager which will maintain
				// a strong reference to this new toolkit.
				TSharedPtr<FDaySequenceEditorToolkit> NewToolkit = MakeShared<FDaySequenceEditorToolkit>(FDaySequenceEditorStyle::Get());
				NewToolkit->InitializeActorPreview(EToolkitMode::WorldCentric, EditWithinLevelEditor, DaySequenceActor.Get());
				NewToolkit->GetSequencer()->OnGlobalTimeChanged().AddRaw(this, &FDaySequenceActorPreview::OnGlobalTimeChanged);
				DaySequencePreviewToolkit = NewToolkit;
				if (LastPreviewTime < 0.0f)
				{
					SetPreviewTime(DaySequenceActor->GetInitialTimeOfDay());
				}
				else
				{
					SetPreviewTime(LastPreviewTime);
				}
				
				// Force update the details panel.
				//
				// This is unfortunately required due to the current sequence of events. Typically,
				// FLevelEditorSequencerIntegration handles the force updating of the details panel.
				// However it only does so in AddSequencer if it no other sequencers are active.
				//
				// - AddSequencer is invoked during Toolkit::InitializeInternal() [ForceDetailsUpdate]
				// - RemoveSequencer is only invoked Toolkit::~Toolkit()
				//
				// Toolkit::OnClose() that fires this delegate is invoked prior to the destructor. As
				// a result, the Sequence Editor toolkit is not yet removed from the FLevelEditorSequencerIntegration
				// and so the forced update of the details panel during AddSequencer is skipped.
				//
				// The main consequence of this is that the transport controls are not regenerated against
				// the newly enabled preview toolkit.
				//
				// TODO: Revisit the order of events to avoid this forced details panel refresh.
				UpdateDetails();
			}
		}
		else
		{
			GEditor->OnBlueprintPreCompile().RemoveAll(this);
			GEditor->OnEditorClose().RemoveAll(this);

			if (const UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (UDaySequenceSubsystem* DaySubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
				{
					DaySubsystem->OnDaySequenceActorSetEvent.RemoveAll(this);
				}
			}

			// Invalidate our level editor viewports to ensure that a tick is queued
			// to process any pending invalidated sequences on the DaySequenceActor.
			UpdateLevelEditorViewports();
		}
	}
}

void FDaySequenceActorPreview::ClosePreviewToolkit()
{
	if (DaySequencePreviewToolkit.IsValid())
	{
		TSharedPtr<FDaySequenceEditorToolkit> Toolkit = DaySequencePreviewToolkit.Pin();
		if (Toolkit->IsActive())
		{
			Toolkit->GetSequencer()->OnGlobalTimeChanged().RemoveAll(this);
			Toolkit->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
		}
	}
}

void FDaySequenceActorPreview::OnBlueprintPreCompile(UBlueprint* Blueprint)
{
	if (IsInActualLoadingThread())
	{
		return;
	}

	UpdateActorPreview();
	
	if (Blueprint && Blueprint->GeneratedClass && DaySequenceActor.IsValid() && DaySequenceActor->IsA(Blueprint->GeneratedClass))
	{
		// Disable the preview prior to recompiling the previewed DaySequenceActor.
		EnablePreview(false);
	}
}

void FDaySequenceActorPreview::OnTimeOfDayPreviewChanged(float PreviewHours)
{
	if (DaySequenceActor.IsValid())
	{
		SetPreviewTime(PreviewHours);
	}
}

void FDaySequenceActorPreview::OnPreRootSequenceChanged()
{
	// Shutdown the preview before the main sequence changes.
	EnablePreview(false);
}

void FDaySequenceActorPreview::OnPostRootSequenceChanged()
{
	// Re-enable the preview with the new main sequence.
	EnablePreview(true);
}

void FDaySequenceActorPreview::OnDaySequenceToolkitPostMapChanged()
{
	// Reset preview time on map change.
	LastPreviewTime = -1.0f;
	
	// Attempt to re-enable the preview on map change after the toolkit has processed
	// its own MapChanged event. This must be done this way because the toolkit MapChanged
	// event closes all open toolkits.
	UpdateActorPreview();
	EnablePreview(true);
}

void FDaySequenceActorPreview::OnMapChanged(uint32 /*MapChangeFlags*/)
{
	// Reset preview time on map change.
	LastPreviewTime = -1.0f;
	
	// Ensure at least one frame tick is performed on map change. This handles
	// the case where the toolkit was never setup (realtime viewport off) and
	// as a result the ToolkitPostMapChanged above is never fired.
	UpdateLevelEditorViewports();
}

void FDaySequenceActorPreview::OnGlobalTimeChanged()
{
	if (DaySequenceActor.IsValid())
	{
		LastPreviewTime = GetPreviewTime();
	}
}

void FDaySequenceActorPreview::OnBeginPIE(bool /*bSimulate*/)
{
	EnablePreview(false);
}

void FDaySequenceActorPreview::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	if (World && World->WorldType == EWorldType::Editor)
	{
		EnablePreview(false);
	}
}

void FDaySequenceActorPreview::UpdateLevelEditorViewports()
{
	if (!GEditor)
	{
		return;
	}
	
	// Redraw if not in PIE/simulate
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;
	if (bIsInPIEOrSimulate)
	{
		return;
	}

	// Request a single real-time frame to be rendered to ensure that we tick the world and update the viewport
	// We only do this on level viewports instead of GetAllViewportClients to avoid needlessly redrawing Cascade,
	// Blueprint, and other editors that have a 3d viewport.
	for (FEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			if (!LevelVC->IsRealtime())
			{
				LevelVC->RequestRealTimeFrames(1);
			}
			LevelVC->Invalidate();
		}
	}
}

void FDaySequenceActorPreview::Register()
{
	FDaySequenceEditorToolkit::OnToolkitPostMapChanged().AddRaw(this, &FDaySequenceActorPreview::OnDaySequenceToolkitPostMapChanged);
	FEditorDelegates::MapChange.AddRaw(this, &FDaySequenceActorPreview::OnMapChanged);
	FEditorDelegates::BeginPIE.AddRaw(this, &FDaySequenceActorPreview::OnBeginPIE);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FDaySequenceActorPreview::Deregister);
	FWorldDelegates::OnWorldCleanup.AddRaw(this, &FDaySequenceActorPreview::OnWorldCleanup);
}

void FDaySequenceActorPreview::Deregister()
{
	FDaySequenceEditorToolkit::OnToolkitPostMapChanged().RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

	if (GEditor)
	{
		if (const UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			if (UDaySequenceSubsystem* DaySubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
			{
				DaySubsystem->OnDaySequenceActorSetEvent.RemoveAll(this);
			}
		}
		
		GEditor->OnBlueprintPreCompile().RemoveAll(this);
		GEditor->OnEditorClose().RemoveAll(this);
	}

	// Do not close preview toolkits here. During OnEditorClose or OnEnginePreExit
	// we cannot assume the availability of dependent systems.
	//
	// Instead rely on the toolkit host or AssetEditorSubsystem to safely
	// shutdown the toolkits. As a result, it is critical to ensure that we
	// no longer Tick past this point.
	//
	// Clear DaySequenceActor to disable our Tick.
	DaySequenceActor.Reset();
}

TSharedRef<SWidget> FDaySequenceActorPreview::MakeTransportControls(bool bExtended) const
{
	const TSharedPtr<ISequencer> Sequencer = DaySequencePreviewToolkit.IsValid() ? DaySequencePreviewToolkit.Pin()->GetSequencer() : nullptr;
	return Sequencer ? Sequencer->MakeTransportControls(bExtended) : SNullWidget::NullWidget;
}

void FDaySequenceActorPreview::UpdateDetails()
{
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	static const FName DetailsTabIdentifiers[] = { "LevelEditorSelectionDetails", "LevelEditorSelectionDetails2", "LevelEditorSelectionDetails3", "LevelEditorSelectionDetails4" };
	for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
	{
		TSharedPtr<IDetailsView> DetailsView = EditModule.FindDetailView(DetailsTabIdentifier);
		if(DetailsView.IsValid())
		{
			DetailsView->ForceRefresh();
		}
	}
}

// There are some workflows (ex. ReplaceReferences, LoadEditorLayout) that force close
// all AssetEditors and rely on the AssetEditorSubsystem to save/reload asset editors back to
// their prior state. Unfortunately the save/reload asset system does not work for transient
// objects which is a use case that DaySequence is designed to support. Consequently, these
// editor actions would forcefully close the DaySequenceActorPreview without re-enabling it
// after the fact.
//
// Also, some delegates (ex. OnCancelPIE) are not opportune moments to enable the
// DaySequenceEditorToolkit since outliner recreation during PIE world shutdown
// can lead to crashes.
//
// As a result, we opt for an editor tick based approach to re-enable the actor preview.
bool FDaySequenceActorPreview::IsTickable() const
{
	return GIsRunning && GEditor && GEditor->PlayWorld == nullptr && !IsPreviewEnabled();
}

TStatId FDaySequenceActorPreview::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDaySequenceActorPreview, STATGROUP_Tickables);
}

void FDaySequenceActorPreview::Tick(float DeltaTime)
{
	if (!DaySequenceActor.IsValid())
	{
		UpdateActorPreview();
	}
	
	if (IsValid() && !FDaySequenceEditorToolkit::HasOpenSequenceEditorToolkits())
	{
		EnablePreview(true);
	}
}
