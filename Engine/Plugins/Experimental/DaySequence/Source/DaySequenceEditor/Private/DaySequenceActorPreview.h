// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"
#include "UObject/WeakObjectPtr.h"

class ADaySequenceActor;
class ISequencer;
class FDaySequenceEditorToolkit;
class SWidget;
class UBlueprint;

/**
 * Mediates the preview of DaySequenceActor sequences.
 */
class FDaySequenceActorPreview : public FTickableEditorObject
{
public:
	virtual ~FDaySequenceActorPreview();
	
	/** @return true if the current state has a valid DaySequence actor. */
	bool IsValid() const;

	/** @return Weak pointer to the DaySequence actor. */
	TWeakObjectPtr<ADaySequenceActor> GetPreviewActor() const;

	/** @return Weak pointer to the sequencer instance hosting the DaySequence actor preview. */
	TWeakPtr<ISequencer> GetPreviewSequencer() const;

	/** @return float, the preview time in hours */
	float GetPreviewTime() const;

	/**
	 * Sets the preview time on the preview sequencer instance.
	 *
	 * @param PreviewTime the time to set in hours.
	 */
	void SetPreviewTime(float PreviewTime);

	/** @return true if there is an active preview. */
	bool IsPreviewEnabled() const;

	/** @return the length of a game day in hours. */
	float GetDayLength() const;

	/**
	 * Enables/disables the actor preview for the current DaySequence actor. This will only
	 * enable actor preview if there are no active Sequence Editor toolkits.
	 *
	 * @param bEnable if true, initializes a new preview, otherwise shuts down any active preview.
	 */
	void EnablePreview(bool bEnable);

	/** Register actor preview state delegates. */
	void Register();

	/** Deregister actor preview state delegates. */
	void Deregister();

	/**
	 * Utility function to generate transport controls for the preview sequencer.
	 *
	 * @param bExtended if true, add extended Sequencer playback controls
	 * @return transport control widget if preview sequencer is valid, SNullWidget otherwise.
	 */
	TSharedRef<SWidget> MakeTransportControls(bool bExtended) const;

protected:
	/** Synchronize the actor preview state to the editor world. */
	void UpdateActorPreview();

	/** Close the active preview toolkit. */
	void ClosePreviewToolkit();

	/**
	 * Invalidate level editor viewports to ensure a tick is fired to process
	 * any invalidated main sequences on the active DaySequenceActor.
	 */
	void UpdateLevelEditorViewports();
	
	void OnTimeOfDayPreviewChanged(float PreviewHours);
	void OnBlueprintPreCompile(UBlueprint* Blueprint);
	void OnPreRootSequenceChanged();
	void OnPostRootSequenceChanged();
	void OnDaySequenceToolkitPostMapChanged();
	void OnMapChanged(uint32 MapChangeFlags);
	void OnGlobalTimeChanged();
	void OnBeginPIE(bool bSimulate);
	void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

private:
	void UpdateDetails();

private:
	TWeakObjectPtr<ADaySequenceActor> DaySequenceActor = nullptr;
	TWeakPtr<FDaySequenceEditorToolkit> DaySequencePreviewToolkit;
	float LastPreviewTime = -1.f;
};
