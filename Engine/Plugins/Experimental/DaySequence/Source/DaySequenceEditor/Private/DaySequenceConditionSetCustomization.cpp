// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceConditionSetCustomization.h"

#include "DetailWidgetRow.h"
#include "EditableDaySequenceConditionSet.h" // IWYU pragma: keep
#include "PropertyHandle.h"
#include "SDaySequenceConditionSetCombo.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FDaySequenceTrackConditionSetCustomization"

TSharedRef<IPropertyTypeCustomization> FDaySequenceConditionSetCustomization::MakeInstance()
{
	return MakeShareable(new FDaySequenceConditionSetCustomization);
}

void FDaySequenceConditionSetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		[
			SNew(SDaySequenceConditionSetCombo)
			.StructPropertyHandle(StructPropertyHandle)
		]
	];
}

void FDaySequenceConditionSetCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{}

#undef LOCTEXT_NAMESPACE
