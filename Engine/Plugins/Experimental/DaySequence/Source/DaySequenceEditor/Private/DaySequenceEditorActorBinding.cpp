// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorActorBinding.h"
#include "DaySequence.h"

#include "ActorTreeItem.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieScenePossessable.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "Selection.h"
#include "Styling/SlateIconFinder.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DaySequenceEditorActorBinding"

FDaySequenceEditorActorBinding::FDaySequenceEditorActorBinding(TSharedRef<ISequencer> InSequencer)
	: Sequencer(InSequencer)
{
}

FText FDaySequenceEditorActorBinding::GetDisplayName() const
{
	return LOCTEXT("DaySequenceEditorActorBinding_DisplayName", "Day Sequence Specialized Binding");
}

void FDaySequenceEditorActorBinding::BuildSequencerAddMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("AddActor_Label", "Actor To Sequencer"),
		LOCTEXT("AddActor_ToolTip", "Allow sequencer to possess an actor that already exists in the current level"),
		FNewMenuDelegate::CreateRaw(this, &FDaySequenceEditorActorBinding::AddPossessActorMenuExtensions),
		false /*bInOpenSubMenuOnClick*/,
		FSlateIcon("DaySequenceStyle", "DaySequenceEditor.PossessNewActor")
		);
}

bool FDaySequenceEditorActorBinding::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return InSequence->GetClass() == UDaySequence::StaticClass();
}

void FDaySequenceEditorActorBinding::AddPossessActorMenuExtensions(FMenuBuilder& MenuBuilder)
{
	// This is called for every actor in the map, and asking the sequencer for a handle to the object to check if we have
	// already bound is an issue on maps that have tens of thousands of actors. The current sequence is will almost always
	// have actors than the map, so instead we'll cache off all of the actors already bound and check against that map locally.
	// This list is checked via an async filter, but we don't need to store them as weak pointers because we're doing a direct
	// pointer comparison and not an object comparison, and the async list shouldn't run the filter if the object is no longer valid.
	// We don't need to check against Sequencer spawnables as they're not valid for possession.
	TSet<UObject*> ExistingPossessedObjects;
	if (Sequencer.IsValid())
	{
		UMovieSceneSequence* MovieSceneSequence = Sequencer.Pin()->GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		if(MovieScene)
		{
			for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); Index++)
			{
				FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
				TArray<UObject*, TInlineAllocator<1>> BoundObjects;
				MovieSceneSequence->LocateBoundObjects(Possessable.GetGuid(), UE::UniversalObjectLocator::FResolveParams(Sequencer.Pin()->GetPlaybackContext()), Sequencer.Pin()->GetSharedPlaybackState(), BoundObjects);

				// A possession guid can apply to more than one object, so we get all bound objects for the GUID and add them to our set.
				ExistingPossessedObjects.Append(BoundObjects);
			}
		}
	}

	auto IsActorValidForPossession = [=](const AActor* InActor, TSet<UObject*> InPossessedObjectSet)
	{
		return !InPossessedObjectSet.Contains((UObject*)InActor);
	};

	// Set up a menu entry to add the selected actor(s) to the sequencer
	TArray<AActor*> ActorsValidForPossession;
	GEditor->GetSelectedActors()->GetSelectedObjects(ActorsValidForPossession);
	ActorsValidForPossession.RemoveAll([&](AActor* In){ return !IsActorValidForPossession(In, ExistingPossessedObjects); });

	FText SelectedLabel;
	FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(AActor::StaticClass());
	if (ActorsValidForPossession.Num() == 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddSpecificActor", "Add '{0}'"), FText::FromString(ActorsValidForPossession[0]->GetActorLabel()));
		ActorIcon = FSlateIconFinder::FindIconForClass(ActorsValidForPossession[0]->GetClass());
	}
	else if (ActorsValidForPossession.Num() > 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddCurrentActorSelection", "Add Current Selection ({0} actors)"), FText::AsNumber(ActorsValidForPossession.Num()));
	}

	if (!SelectedLabel.IsEmpty())
	{
		// Copy the array into the lambda - probably not that big a deal
		MenuBuilder.AddMenuEntry(SelectedLabel, FText(), ActorIcon, FExecuteAction::CreateLambda([this, ActorsValidForPossession]{
			FSlateApplication::Get().DismissAllMenus();
			AddActorsToSequencer(ActorsValidForPossession.GetData(), ActorsValidForPossession.Num());
		}));
	}

	// Add an entry for an empty binding
	MenuBuilder.AddMenuEntry(LOCTEXT("EmptyBinding", "New Empty Binding"),
		LOCTEXT("EmptyBindingTooltip", "Add a new empty binding to Sequencer which can be connected to an object or actor afterwards in the Binding Properties"),
		FSlateIcon(), // TODO: empty icon?
		FExecuteAction::CreateLambda([this] {
			FSlateApplication::Get().DismissAllMenus();
			Sequencer.Pin()->AddEmptyBinding();
			}));
	
	MenuBuilder.BeginSection("ChooseActorSection", LOCTEXT("ChooseActor", "Choose Actor:"));

	// Set up a menu entry to add any arbitrary actor to the sequencer
	FSceneOutlinerInitializationOptions InitOptions;
	{
		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.bShowTransient = true;

		// Only want the actor label column
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		// Only display actors that are not possessed already
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(IsActorValidForPossession, ExistingPossessedObjects));
	}

	// actor selector to allow the user to choose an actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef< SWidget > MiniSceneOutliner =
		SNew(SBox)
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([this](AActor* Actor){
					// Create a new binding for this actor
					FSlateApplication::Get().DismissAllMenus();
					AddActorsToSequencer(&Actor, 1);
				})
			)
		];

	MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);
	MenuBuilder.EndSection();
}

void FDaySequenceEditorActorBinding::AddActorsToSequencer(AActor*const* InActors, int32 NumActors)
{
	TArray<TWeakObjectPtr<AActor>> Actors;

	while (NumActors--)
	{
		AActor* ThisActor = *InActors;
		Actors.Add(ThisActor);

		InActors++;
	}

	Sequencer.Pin()->AddActors(Actors);
}

#undef LOCTEXT_NAMESPACE
