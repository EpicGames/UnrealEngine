// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerEditorObjectBinding.h"
#include "Templates/SharedPointer.h"

class ISequencer;
class AActor;

class FDaySequenceEditorActorBinding : public ISequencerEditorObjectBinding
{
public:

	FDaySequenceEditorActorBinding(TSharedRef<ISequencer> InSequencer);

	// ISequencerEditorObjectBinding interface
	virtual FText GetDisplayName() const override;
	virtual void BuildSequencerAddMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

private:

	/** Menu extension callback for the add menu */
	void AddPossessActorMenuExtensions(FMenuBuilder& MenuBuilder);
	
	/** Add the specified actors to the sequencer */
	void AddActorsToSequencer(AActor*const* InActors, int32 NumActors);

private:
	TWeakPtr<ISequencer> Sequencer;
};
