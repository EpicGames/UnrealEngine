// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorCommands.h"

#define LOCTEXT_NAMESPACE "FDaySequenceEditorModule"

void FDaySequenceEditorCommands::RegisterCommands()
{
	UI_COMMAND(OverrideInitialTimeOfDay, "Override Initial Time of Day", "Use the current preview time as the initial time of day in PIE", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(OverrideRunDayCycle, "Freeze Time in PIE", "Pauses the day cycle when starting PIE", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(SelectDaySequenceActor, "Select Day Sequence Actor", "Selects the Day Sequence Actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RefreshDaySequenceActor, "Refresh Day Sequence Actor", "Refreshes the Main Sequence in the Day Sequence Actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenDaySequenceActor, "Open Day Sequence Actor", "Opens the Day Sequence Actor blueprint", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenRootSequence, "Open Root Sequence", "Opens the root sequence on the Day Sequence Actor", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SnapSectionsToTimelineUsingSourceTimecode, "Snap Sections to Timeline using Source Timecode", "Snap selected sections to the position in the timeline matching their source timecode", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SyncSectionsUsingSourceTimecode, "Sync Sections using Source Timecode", "Synchronize sections to the first selected section using source timecode", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BakeTransform, "Bake Transform", "Bake transform in world space, removing any existing transform and attach tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FixActorReferences, "Fix Actor References", "Try to automatically fix up broken actor bindings", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddActorsToBinding, "Add Selected", "Add selected actors to this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveActorsFromBinding, "Remove Selected", "Remove selected actors from this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReplaceBindingWithActors, "Replace with Selected", "Replace the object binding with selected actors", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllBindings, "Remove All", "Remove all bound actors from this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveInvalidBindings, "Remove Missing", "Remove missing objects bound to this track", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
