// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "DaySequenceEditorStyle.h"

class FDaySequenceEditorCommands : public TCommands<FDaySequenceEditorCommands>
{
public:

	FDaySequenceEditorCommands()
		: TCommands<FDaySequenceEditorCommands>(TEXT("DaySequence"), NSLOCTEXT("Contexts", "DaySequence", "DaySequence Plugin"), NAME_None, FDaySequenceEditorStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> OverrideInitialTimeOfDay;
	TSharedPtr<FUICommandInfo> OverrideRunDayCycle;
	
	TSharedPtr<FUICommandInfo> SelectDaySequenceActor;
	TSharedPtr<FUICommandInfo> RefreshDaySequenceActor;
	TSharedPtr<FUICommandInfo> OpenDaySequenceActor;
	TSharedPtr<FUICommandInfo> OpenRootSequence;
	
	TSharedPtr<FUICommandInfo> SnapSectionsToTimelineUsingSourceTimecode;
	TSharedPtr<FUICommandInfo> SyncSectionsUsingSourceTimecode;

	TSharedPtr<FUICommandInfo> BakeTransform;
	TSharedPtr<FUICommandInfo> FixActorReferences;

	TSharedPtr<FUICommandInfo> AddActorsToBinding;
	TSharedPtr<FUICommandInfo> RemoveActorsFromBinding;
	TSharedPtr<FUICommandInfo> ReplaceBindingWithActors;
	TSharedPtr<FUICommandInfo> RemoveAllBindings;
	TSharedPtr<FUICommandInfo> RemoveInvalidBindings;
};
