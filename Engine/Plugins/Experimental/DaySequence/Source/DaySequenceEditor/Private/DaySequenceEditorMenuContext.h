// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Package.h"
#include "DaySequenceEditorMenuContext.generated.h"

class FDaySequenceEditorToolkit;

UCLASS()
class UDaySequenceEditorMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TWeakPtr<FDaySequenceEditorToolkit> Toolkit;
};
