// Copyright Epic Games, Inc. All Rights Reserved.

#include "IDaySequenceEditorModule.h"
#include "DaySequenceEditorStyle.h"
#include "DaySequenceEditorCommands.h"
#include "DaySequenceEditorSettings.h"
#include "DaySequenceActor.h"
#include "DaySequenceActorDetails.h"
#include "DaySequenceActorPreview.h"
#include "DaySequenceEditorActorBinding.h"
#include "DaySequenceEditorActorSpawner.h"
#include "DaySequenceEditorSpecializedBinding.h"
#include "DaySequenceSubsystem.h"
#include "DaySequenceConditionSetCustomization.h"
#include "DaySequenceTimeDetailsCustomization.h"
#include "DaySequenceEditorToolkit.h"
#include "DaySequenceTrackEditor.h"
#include "DaySequenceModifierComponent.h"
#include "Editor/UnrealEdTypes.h"
#include "MovieSceneSequenceEditor_DaySequence.h"
#include "IDaySequenceModule.h"
#include "EnvironmentLightingActorDetails.h"

#include "ISequencerModule.h"
#include "ISettingsModule.h"
#include "SequencerSettings.h"
#include "Sections/MovieSceneSubSection.h"

#include "Application/ThrottleManager.h"
#include "Framework/Docking/TabManager.h"
#include "ToolMenu.h"
#include "ViewportToolBarContext.h"
#include "PropertyEditorModule.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "SEditorViewportToolBarMenu.h"
#include "Selection.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SNullWidget.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "DaySequenceEditor"

static const FName PropertyEditorModuleName("PropertyEditor");
static const FName DaySequenceActorClassName("DaySequenceActor");
static const FName DaySequenceConditionSetName("DaySequenceConditionSet");
static const FName DaySequenceTimeName("DaySequenceTime");
static const FName DaySequenceViewportToolBarExtensionName("DaySequenceEditorViewportToolBar");
static const FName EnvironmentLightingActorClassName("EnvironmentLightingActor");

namespace UE::DaySequence
{
	static TAutoConsoleVariable<bool> CVarShowToolbarMenuLabel(
		TEXT("DaySequence.ToolbarMenu.ShowLabel"),
		true,
		TEXT("When true, the Time of Day toolbar menu will have a label. When false, only an icon will be shown."),
		ECVF_Default
	);
}

class FDaySequenceEditorModule : public IDaySequenceEditorModule, public FGCObject
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override;

	// IDaySequenceEditorModule interface
	virtual FAllowPlaybackContext& OnComputePlaybackContext() override;
	virtual FDaySequenceActorPreview& GetDaySequenceActorPreview() override;
	virtual FPreSelectDaySequenceActor& OnPreSelectDaySequenceActor() override;
	virtual FPostSelectDaySequenceActor& OnPostSelectDaySequenceActor() override;
	
	bool IsDaySequenceActorPreviewEnabled() const;
	void OnOpenRootSequence();
	bool CanOpenRootSequence() const;
	void OnSelectDaySequenceActor();
	bool CanSelectDaySequenceActor() const;
	void OnRefreshDaySequenceActor();
	bool CanRefreshDaySequenceActor() const;
	void OnOpenDaySequenceActor() const;
	bool CanOpenDaySequenceActor() const;
	static TSharedRef<ISequencerEditorObjectBinding> OnCreateActorBinding(TSharedRef<ISequencer> InSequencer);
	static TSharedRef<ISequencerEditorObjectBinding> OnCreateSpecializedBinding(TSharedRef<ISequencer> InSequencer);
	
	TSharedPtr<FPropertySection> RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName);
	void RegisterModulePropertySections();
	void DeregisterModulePropertySections();

private:
	void RegisterMenus();
	void DeregisterMenus();
	
	void RegisterEditorObjectBindings();
	void DeregisterEditorObjectBindings();

	void RegisterEditorActorSpawner();
	void DeregisterEditorActorSpawner();

	void RegisterSequenceEditor();
	void DeregisterSequenceEditor();

	void RegisterSettings();
	void DeregisterSettings();

	void CreateDaySequenceViewportMenu(UToolMenu* Menu);

	/** Shared widget functions */
	static bool IsMenuVisible();
	TSharedRef<SWidget> CreateTimeOfDayWidget() const;
	
	TSharedRef<SWidget> CreateDaySequencePreviewWidget();

	void OnEditorCameraMoved(const FVector& Location, const FRotator& Rotation, ELevelViewportType Type, int32 ViewIndex);
	
	TSharedPtr<class FUICommandList> PluginCommands;

	TMultiMap<FName, FName> RegisteredPropertySections;

	FAllowPlaybackContext OnComputePlaybackContextDelegate;

	FPreSelectDaySequenceActor OnPreSelectDaySequenceActorDelegate;
	FPostSelectDaySequenceActor OnPostSelectDaySequenceActorDelegate;

	FDelegateHandle ActorBindingDelegateHandle;

	FDelegateHandle SpecializedBindingDelegateHandle;

	FDelegateHandle EditorActorSpawnerDelegateHandle;

	FDelegateHandle SequenceEditorHandle;

	FDelegateHandle DaySequenceTrackCreateEditorHandle;
	
	TObjectPtr<USequencerSettings> Settings = nullptr;

	FDaySequenceActorPreview DaySequenceActorPreview;

	FDelegateHandle OnEditorCameraMovedHandle;

	FDelegateHandle OnBeginPIEHandle;
	
	FDelegateHandle OnEndPIEHandle;

	FDelegateHandle OnSwitchPIEAndSIEHandle;

	FDelegateHandle OnSubSectionRemovedHandle;

	int32 LastAllowThrottling = 1;
};

void FDaySequenceEditorModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FDaySequenceEditorStyle::Initialize();
	FDaySequenceEditorCommands::Register();

	OnEditorCameraMovedHandle = FEditorDelegates::OnEditorCameraMoved.AddRaw(this, &FDaySequenceEditorModule::OnEditorCameraMoved);

	OnBeginPIEHandle = FEditorDelegates::BeginPIE.AddLambda([](bool bIsSimulating)
	{
		UDaySequenceModifierComponent::SetIsSimulating(bIsSimulating);
	});
	
	OnEndPIEHandle = FEditorDelegates::EndPIE.AddLambda([](bool)
	{
		UDaySequenceModifierComponent::SetIsSimulating(false);
	});

	OnSwitchPIEAndSIEHandle = FEditorDelegates::OnSwitchBeginPIEAndSIE.AddLambda([](bool bIsSimulating)
	{
		UDaySequenceModifierComponent::SetIsSimulating(bIsSimulating);
	});

	OnSubSectionRemovedHandle = ADaySequenceActor::OnSubSectionRemovedEvent.AddLambda([](const UMovieSceneSubSection* RemovedSubSection)
	{
		FDaySequenceEditorToolkit::IterateOpenToolkits([RemovedSubSection](const FDaySequenceEditorToolkit& Toolkit)
		{
			if (!Toolkit.IsActorPreview())
			{
				// In rare cases a focused subsection can be removed right as an evaluation is triggered,
				// breaking assumptions in core Sequencer code that the focused subsequence is always valid.
				// Any time a subsection is removed we should enforce this assumption.
				if (const TSharedPtr<ISequencer> ToolkitSequencer = Toolkit.GetSequencer())
				{
					const UMovieSceneSequence* FocusedSequence = ToolkitSequencer->GetFocusedMovieSceneSequence();
					const UMovieSceneSequence* RemovedSequence = RemovedSubSection ? RemovedSubSection->GetSequence() : nullptr;
					
					if (!FocusedSequence || FocusedSequence == RemovedSequence)
					{
						ToolkitSequencer->PopToSequenceInstance(ToolkitSequencer->GetRootTemplateID());
					}
				}
				
				// Break out of iteration now that we found a sequence editor
				return false;
			}

			// Continue looking for sequence editor toolkit
			return true;
		});
	});
	
	PluginCommands = MakeShareable(new FUICommandList);

	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().OverrideInitialTimeOfDay,
		FExecuteAction::CreateLambda([this]()
		{
			// Propagate to listeners if bOverrideInitialTimeOfDay is toggled. Preview time is unchanged.
			if (ADaySequenceActor* PreviewActor = DaySequenceActorPreview.GetPreviewActor().Get())
			{
				PreviewActor->SetOverrideInitialTimeOfDay(!PreviewActor->GetOverrideInitialTimeOfDay());
			}
		}),
		FCanExecuteAction::CreateLambda([](){ return true; }),
		FIsActionChecked::CreateLambda([this]()
		{
			// Poll for bOverrideInitialTimeOfDay
			if (const ADaySequenceActor* PreviewActor = DaySequenceActorPreview.GetPreviewActor().Get())
			{
				return PreviewActor->GetOverrideInitialTimeOfDay();
			}

			return false;
		}));
	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().OverrideRunDayCycle,
		FExecuteAction::CreateLambda([this]()
		{
			// Propagate to listeners if bOverrideRunDayCycle is toggled.
			if (ADaySequenceActor* PreviewActor = DaySequenceActorPreview.GetPreviewActor().Get())
			{
				PreviewActor->SetOverrideRunDayCycle(!PreviewActor->GetOverrideRunDayCycle());
			}
		}),
		FCanExecuteAction::CreateLambda([](){ return true; }),
		FIsActionChecked::CreateLambda([this]()
		{
			// Poll for bOverrideRunDayCycle
			if (const ADaySequenceActor* PreviewActor = DaySequenceActorPreview.GetPreviewActor().Get())
			{
				return PreviewActor->GetOverrideRunDayCycle();
			}

			return false;
		}));
	
	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().OpenRootSequence,
		FExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::OnOpenRootSequence),
		FCanExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::CanOpenRootSequence));
	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().SelectDaySequenceActor,
		FExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::OnSelectDaySequenceActor),
		FCanExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::CanSelectDaySequenceActor));
	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().RefreshDaySequenceActor,
		FExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::OnRefreshDaySequenceActor),
		FCanExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::CanRefreshDaySequenceActor));
	PluginCommands->MapAction(
		FDaySequenceEditorCommands::Get().OpenDaySequenceActor,
		FExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::OnOpenDaySequenceActor),
		FCanExecuteAction::CreateRaw(this, &FDaySequenceEditorModule::CanOpenDaySequenceActor));

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FDaySequenceEditorModule::RegisterMenus));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	PropertyModule.RegisterCustomClassLayout(DaySequenceActorClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FDaySequenceActorDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(EnvironmentLightingActorClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FEnvironmentLightingActorDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(DaySequenceConditionSetName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDaySequenceConditionSetCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(DaySequenceTimeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDaySequenceTimeDetailsCustomization::MakeInstance));
	
	RegisterModulePropertySections();

	RegisterEditorObjectBindings();

	RegisterEditorActorSpawner();

	RegisterSettings();

	RegisterSequenceEditor();

	DaySequenceActorPreview.Register();
}

void FDaySequenceEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FEditorDelegates::OnEditorCameraMoved.Remove(OnEditorCameraMovedHandle);

	FEditorDelegates::BeginPIE.Remove(OnBeginPIEHandle);
	
	FEditorDelegates::EndPIE.Remove(OnEndPIEHandle);

	FEditorDelegates::OnSwitchBeginPIEAndSIE.Remove(OnSwitchPIEAndSIEHandle);

	ADaySequenceActor::OnSubSectionRemovedEvent.Remove(OnSubSectionRemovedHandle);

	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditorModuleName))
	{
		PropertyModule->UnregisterCustomClassLayout(DaySequenceActorClassName);
		PropertyModule->UnregisterCustomClassLayout(EnvironmentLightingActorClassName);
		PropertyModule->UnregisterCustomPropertyTypeLayout(DaySequenceConditionSetName);
	}

	DaySequenceActorPreview.Deregister();
	
	DeregisterSequenceEditor();

	DeregisterSettings();

	DeregisterEditorActorSpawner();

	DeregisterEditorObjectBindings();

	DeregisterModulePropertySections();

	DeregisterMenus();

	UToolMenus::UnRegisterStartupCallback(this);

	UToolMenus::UnregisterOwner(this);

	FDaySequenceEditorStyle::Shutdown();

	FDaySequenceEditorCommands::Unregister();
}

void FDaySequenceEditorModule::AddReferencedObjects( FReferenceCollector& Collector )
{
	if (Settings)
	{
		Collector.AddReferencedObject(Settings);
	}
}

FString FDaySequenceEditorModule::GetReferencerName() const
{
	return "FDaySequenceEditorModule";
}

FDaySequenceEditorModule::FAllowPlaybackContext& FDaySequenceEditorModule::OnComputePlaybackContext()
{
	return OnComputePlaybackContextDelegate;
}

FDaySequenceActorPreview& FDaySequenceEditorModule::GetDaySequenceActorPreview()
{
	return DaySequenceActorPreview;
}

IDaySequenceEditorModule::FPreSelectDaySequenceActor& FDaySequenceEditorModule::OnPreSelectDaySequenceActor()
{
	return OnPreSelectDaySequenceActorDelegate;
}

IDaySequenceEditorModule::FPostSelectDaySequenceActor& FDaySequenceEditorModule::OnPostSelectDaySequenceActor()
{
	return OnPostSelectDaySequenceActorDelegate;
}

bool FDaySequenceEditorModule::IsDaySequenceActorPreviewEnabled() const
{
	return DaySequenceActorPreview.IsPreviewEnabled();
}

void FDaySequenceEditorModule::OnOpenRootSequence()
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		if (ADaySequenceActor* DayActor = DaySubsystem->GetDaySequenceActor())
		{
			if (UObject* LoadedObject = DayActor->GetRootSequence())
			{
				FFrameTime InitialGlobalTime;
				
				// Disable the preview prior to opening the root sequence otherwise
				// the preview toolkit will be returned as the active editor for this
				// root sequence asset.
				if (DaySequenceActorPreview.IsPreviewEnabled())
				{
					InitialGlobalTime = DaySequenceActorPreview.GetPreviewSequencer().Pin()->GetGlobalTime().Time;
					DaySequenceActorPreview.EnablePreview(false);
				}
				
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LoadedObject);

				// This iteration does work necessary to keep the viewport preview time consistent when opening/closing a sequence editor.
				FDaySequenceEditorToolkit::IterateOpenToolkits([InitialGlobalTime](FDaySequenceEditorToolkit& Toolkit)
				{
					if (!Toolkit.IsActorPreview())
					{
						// One time propagation of preview actor time for initializing the sequence editor
						const TSharedPtr<ISequencer> ToolkitSequencer = Toolkit.GetSequencer();
						ToolkitSequencer->SetGlobalTime(InitialGlobalTime);
						
						// Break out of iteration now that we found a sequence editor
						return false;
					}

					// Continue looking for sequence editor toolkit
					return true;
				});
			}
		}
	}
}

bool FDaySequenceEditorModule::CanOpenRootSequence() const
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		const ADaySequenceActor* DayActor = DaySubsystem->GetDaySequenceActor();
		return DayActor && DayActor->GetRootSequence();
	}
	return false;
}

void FDaySequenceEditorModule::OnSelectDaySequenceActor()
{
	OnPreSelectDaySequenceActorDelegate.Broadcast();
	
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		GEditor->GetSelectedActors()->Modify();
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(DaySubsystem->GetDaySequenceActor(), true, true);
	}

	OnPostSelectDaySequenceActorDelegate.Broadcast();
}

bool FDaySequenceEditorModule::CanSelectDaySequenceActor() const
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		const ADaySequenceActor* DayActor = DaySubsystem->GetDaySequenceActor();
		return DayActor != nullptr;
	}
	return false;
}

void FDaySequenceEditorModule::OnRefreshDaySequenceActor()
{
	const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
	if (ADaySequenceActor* DayActor = DaySubsystem->GetDaySequenceActor())
	{
		DayActor->UpdateRootSequence(ADaySequenceActor::EUpdateRootSequenceMode::Reinitialize);
	}
}

bool FDaySequenceEditorModule::CanRefreshDaySequenceActor() const
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		return DaySubsystem->GetDaySequenceActor() != nullptr;
	}
	return false;
}

void FDaySequenceEditorModule::OnOpenDaySequenceActor() const
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		const ADaySequenceActor* DayActor = DaySubsystem ? DaySubsystem->GetDaySequenceActor() : nullptr;
		const UClass* DayClass = DayActor ? DayActor->GetClass() : nullptr;
		const UBlueprint* DayBlueprint = DayClass ? Cast<UBlueprint>(DayClass->ClassGeneratedBy) : nullptr;
		if (DayBlueprint)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DayBlueprint);
		}
	}
}

bool FDaySequenceEditorModule::CanOpenDaySequenceActor() const
{
	if (GEditor)
	{
		const UDaySequenceSubsystem* DaySubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		const ADaySequenceActor* DayActor = DaySubsystem->GetDaySequenceActor();
		return DayActor && DayActor->GetClass() && Cast<UBlueprint>(DayActor->GetClass()->ClassGeneratedBy);
	}
	return false;
}

TSharedPtr<FPropertySection> FDaySequenceEditorModule::RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName)
{
	TSharedRef<FPropertySection> PropertySection = PropertyModule.FindOrCreateSection(ClassName, SectionName, DisplayName);
	RegisteredPropertySections.Add(ClassName, SectionName);
	return PropertySection;
}

void FDaySequenceEditorModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);
	{
		{
			UToolMenu* ViewportMenu = UToolMenus::Get()->ExtendMenu("DaySequence.ViewportToolBar");
			ViewportMenu->AddDynamicSection("DynamicSection", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				CreateDaySequenceViewportMenu(InMenu);
			}));
		}
		
		{
			UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolBar");
			
			ToolbarMenu->FindOrAddSection("Left").AddEntry(FToolMenuEntry::InitComboButton(
				"TimeOfDay",
				FToolUIActionChoice(FUIAction(FExecuteAction(), FCanExecuteAction(), FGetActionCheckState(), FIsActionButtonVisible::CreateLambda([]() { return IsMenuVisible(); }))),
				FNewToolMenuChoice(FOnGetContent::CreateRaw(this, &FDaySequenceEditorModule::CreateTimeOfDayWidget)),
				TAttribute<FText>::CreateLambda([]() { return UE::DaySequence::CVarShowToolbarMenuLabel.GetValueOnAnyThread() ? LOCTEXT("DaySequenceMenuLabel", "Time of Day") : FText::GetEmpty(); }),
				FText::GetEmpty(),
				FSlateIcon(FDaySequenceEditorStyle::GetStyleSetName(), "DaySequenceEditor.ViewportToolBar")
			));
		}
	}
}

void FDaySequenceEditorModule::DeregisterMenus()
{
	if (GEditor)
	{
		if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
		{
			PanelExtensionSubsystem->UnregisterPanelFactory(DaySequenceViewportToolBarExtensionName);
		}
	}
}

void FDaySequenceEditorModule::CreateDaySequenceViewportMenu(UToolMenu* Menu)
{
	{
		FToolMenuSection& Section = Menu->AddSection("DaySequencePreview", LOCTEXT("DaySequencePreviewHeader", "Preview"));
		Section.AddEntry(FToolMenuEntry::InitWidget("DaySequencePreviewTime", CreateDaySequencePreviewWidget(), LOCTEXT("DaySequencePreviewTime","Time")));
	}
	{
		FToolMenuSection& Section = Menu->AddSection("DaySequencePIESettings", LOCTEXT("DaySequencePIESettingsHeader", "PIE Settings"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().OverrideInitialTimeOfDay, PluginCommands));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().OverrideRunDayCycle, PluginCommands));
	}
	{
		FToolMenuSection& Section = Menu->AddSection("DaySequenceActions", LOCTEXT("DaySequenceActionsHeader", "Actions"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().OpenRootSequence, PluginCommands));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().SelectDaySequenceActor, PluginCommands));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().RefreshDaySequenceActor, PluginCommands));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(FDaySequenceEditorCommands::Get().OpenDaySequenceActor, PluginCommands));
	}
}

bool FDaySequenceEditorModule::IsMenuVisible()
{
	bool Result = false;
	if (!UE::GetIsEditorLoadingPackage() && GEditor)
	{
		if (const UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			const UDaySequenceSubsystem* TODSubsystem = World->GetSubsystem<UDaySequenceSubsystem>();
			Result = TODSubsystem && TODSubsystem->GetDaySequenceActor();
		}
	}
	return Result;
}

TSharedRef<SWidget> FDaySequenceEditorModule::CreateTimeOfDayWidget() const
{
	static const FName MenuName("DaySequence.ViewportToolBar");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenus::Get()->RegisterMenu(MenuName);
	}

	const FToolMenuContext MenuContext(PluginCommands, TSharedPtr<FExtender>());
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

TSharedRef<SWidget> FDaySequenceEditorModule::CreateDaySequencePreviewWidget()
{
	return
		SNew( SBox )
		.HAlign( HAlign_Left )
		[
			SNew( SBox )
			.Padding( FMargin(4.0f, 0.0f, 0.0f, 0.0f) )
			.WidthOverride( 240.0f )
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
					.IsEnabled_Lambda([this]()
					{
						return IsDaySequenceActorPreviewEnabled();
					})
					.Delta(0.03125f) // 1/32
					.MinValue(0.f)
					.MaxValue(DaySequenceActorPreview.GetDayLength())
					.Value_Lambda([this]()
					{
						return DaySequenceActorPreview.GetPreviewTime();
					})
					.OnValueChanged_Lambda([this](float NewValue)
					{
						if (ADaySequenceActor* PreviewActor = DaySequenceActorPreview.GetPreviewActor().Get())
                        {
							// Updates PreviewActor's preview time (which will then update DaySequenceActorPreview's preview time) and broadcasts OnOverrideInitialTimeOfDayChanged.
							PreviewActor->SetOverrideInitialTimeOfDay(PreviewActor->GetOverrideInitialTimeOfDay(), NewValue);
                        }
					})
					.OnBeginSliderMovement_Lambda([this]()
					{
						// Disable Slate throttling during slider drag to ensure immediate
                    	// Lumen updates while scrubbing the time.
						FSlateThrottleManager::Get().DisableThrottle(true);
					})
					.OnEndSliderMovement_Lambda([this](float)
					{
						FSlateThrottleManager::Get().DisableThrottle(false);
					})
				]
			]
		];
}

void FDaySequenceEditorModule::RegisterModulePropertySections()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	{
		TSharedPtr<FPropertySection> Section = RegisterPropertySection(PropertyModule, "DaySequenceActor", "General", LOCTEXT("General", "General"));
		Section->AddCategory("Sequence");
		Section->AddCategory("Preview");
		Section->AddCategory("RuntimeDayCycle");
		Section->AddCategory("BindingOverrides");
	}
}

void FDaySequenceEditorModule::DeregisterModulePropertySections()
{
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditorModuleName);
	if (!PropertyModule)
	{
		return;
	}

	for (TMultiMap<FName, FName>::TIterator PropertySectionIterator = RegisteredPropertySections.CreateIterator(); PropertySectionIterator; ++PropertySectionIterator)
	{
		PropertyModule->RemoveSection(PropertySectionIterator->Key, PropertySectionIterator->Value);
		PropertySectionIterator.RemoveCurrent();
	}
}

/** Register sequencer editor object bindings */
void FDaySequenceEditorModule::RegisterEditorObjectBindings()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	ActorBindingDelegateHandle = SequencerModule.RegisterEditorObjectBinding(FOnCreateEditorObjectBinding::CreateStatic(&FDaySequenceEditorModule::OnCreateActorBinding));
	SpecializedBindingDelegateHandle = SequencerModule.RegisterEditorObjectBinding(FOnCreateEditorObjectBinding::CreateStatic(&FDaySequenceEditorModule::OnCreateSpecializedBinding));

}

/** Unregisters sequencer editor object bindings */
void FDaySequenceEditorModule::DeregisterEditorObjectBindings()
{
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule)
	{
		SequencerModule->UnRegisterEditorObjectBinding(ActorBindingDelegateHandle);
		SequencerModule->UnRegisterEditorObjectBinding(SpecializedBindingDelegateHandle);
	}
}

void FDaySequenceEditorModule::RegisterEditorActorSpawner()
{
	IDaySequenceModule& DaySequenceModule = FModuleManager::LoadModuleChecked<IDaySequenceModule>("DaySequence");
	EditorActorSpawnerDelegateHandle = DaySequenceModule.RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner::CreateStatic(&FDaySequenceEditorActorSpawner::CreateObjectSpawner));
}

void FDaySequenceEditorModule::DeregisterEditorActorSpawner()
{
	IDaySequenceModule* DaySequenceModule = FModuleManager::GetModulePtr<IDaySequenceModule>("DaySequence");
	if (DaySequenceModule)
	{
		DaySequenceModule->UnregisterObjectSpawner(EditorActorSpawnerDelegateHandle);
	}
}


TSharedRef<ISequencerEditorObjectBinding> FDaySequenceEditorModule::OnCreateActorBinding(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FDaySequenceEditorActorBinding(InSequencer));
}

TSharedRef<ISequencerEditorObjectBinding> FDaySequenceEditorModule::OnCreateSpecializedBinding(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FDaySequenceEditorSpecializedBinding(InSequencer));
}

void FDaySequenceEditorModule::RegisterSequenceEditor()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	SequenceEditorHandle = SequencerModule.RegisterSequenceEditor(UDaySequence::StaticClass(), MakeUnique<FMovieSceneSequenceEditor_DaySequence>());

	DaySequenceTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FDaySequenceTrackEditor::CreateTrackEditor ) );
}

void FDaySequenceEditorModule::DeregisterSequenceEditor()
{
	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule)
	{
		SequencerModule->UnregisterSequenceEditor(SequenceEditorHandle);

		SequencerModule->UnRegisterTrackEditor( DaySequenceTrackCreateEditorHandle );
	}
}

/** Register settings objects. */
void FDaySequenceEditorModule::RegisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "DaySequenceEditor",
			LOCTEXT("DaySequenceEditorProjectSettingsName", "Day Sequence Editor"),
			LOCTEXT("DaySequenceEditorProjectSettingsDescription", "Configure the Day Sequence Editor."),
			GetMutableDefault<UDaySequenceEditorSettings>()
		);

		Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("DaySequenceEditor"));

		SettingsModule->RegisterSettings("Editor", "ContentEditors", "DaySequenceEditor",
			LOCTEXT("DaySequenceEditorSettingsName", "Day Sequence Editor"),
			LOCTEXT("DaySequenceEditorSettingsDescription", "Configure the look and feel of the Day Sequence Editor."),
			Settings);
	}
}

/** Deregister settings objects. */
void FDaySequenceEditorModule::DeregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "DaySequenceEditor");
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "DaySequenceEditor");
	}
}

void FDaySequenceEditorModule::OnEditorCameraMoved(const FVector& Location, const FRotator& Rotation, ELevelViewportType Type, int32 ViewIndex)
{
	if (Type == LVT_Perspective)
	{
		UDaySequenceModifierComponent::SetVolumePreviewLocation(Location);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDaySequenceEditorModule, DaySequenceEditor)
