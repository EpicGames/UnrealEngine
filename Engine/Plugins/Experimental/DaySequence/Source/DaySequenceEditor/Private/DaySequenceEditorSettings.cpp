// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceEditorSettings)

UDaySequenceEditorSettings::UDaySequenceEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoBindToSimulate = true;
	bAutoBindToPIE      = true;
}


