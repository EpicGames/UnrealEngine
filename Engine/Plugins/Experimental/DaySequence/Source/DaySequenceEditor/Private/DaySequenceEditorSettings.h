// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "DaySequenceEditorSettings.generated.h"

USTRUCT()
struct FDaySequencePropertyTrackSettings
{
	GENERATED_BODY()

	/** Optional ActorComponent tag (when keying a component property). */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString ComponentPath;

	/** Path to the keyed property within the Actor or ActorComponent. */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString PropertyPath;
};


USTRUCT()
struct FDaySequenceTrackSettings
{
	GENERATED_BODY()

	/** The Actor class to create movie scene tracks for. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/Engine.Actor"))
	FSoftClassPath MatchingActorClass;

	/** List of movie scene track classes to be added automatically. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/MovieScene.MovieSceneTrack"))
	TArray<FSoftClassPath> DefaultTracks;

	/** List of movie scene track classes not to be added automatically. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/MovieScene.MovieSceneTrack"))
	TArray<FSoftClassPath> ExcludeDefaultTracks;

	/** List of property names for which movie scene tracks will be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	TArray<FDaySequencePropertyTrackSettings> DefaultPropertyTracks;

	/** List of property names for which movie scene tracks will not be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	TArray<FDaySequencePropertyTrackSettings> ExcludeDefaultPropertyTracks;
};


/**
 * DaySequence Editor settings.
 */
UCLASS(config=EditorPerProjectUserSettings)
class UDaySequenceEditorSettings
	: public UObject
{
	GENERATED_BODY()

public:

	UDaySequenceEditorSettings(const FObjectInitializer& ObjectInitializer);

	/** Specifies class properties for which movie scene tracks will be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=Tracks)
	TArray<FDaySequenceTrackSettings> TrackSettings;

	/** Specifies whether to automatically bind an active sequencer UI to PIE worlds. */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bAutoBindToPIE;

	/** Specifies whether to automatically bind an active sequencer UI to simulate worlds. */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bAutoBindToSimulate;
};


