// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorSpawnRegister.h"

#include "AssetRegistry/AssetData.h"
#include "Components/ActorComponent.h"
#include "Editor.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "LevelEditor.h"
#include "MovieSceneSpawnable.h"
#include "UObject/ObjectSaveContext.h"
#include "Selection.h"
#include "TransformData.h"

#define LOCTEXT_NAMESPACE "DaySequenceEditorSpawnRegister"

/* FDaySequenceEditorSpawnRegister structors
 *****************************************************************************/

FDaySequenceEditorSpawnRegister::FDaySequenceEditorSpawnRegister()
{
	bShouldClearSelectionCache = true;

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	OnActorSelectionChangedHandle = LevelEditor.OnActorSelectionChanged().AddRaw(this, &FDaySequenceEditorSpawnRegister::HandleActorSelectionChanged);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FDaySequenceEditorSpawnRegister::OnObjectsReplaced);

	OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FDaySequenceEditorSpawnRegister::OnObjectModified);
	OnObjectSavedHandle    = FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FDaySequenceEditorSpawnRegister::OnPreObjectSaved);
#endif
}


FDaySequenceEditorSpawnRegister::~FDaySequenceEditorSpawnRegister()
{
	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnActorSelectionChanged().Remove(OnActorSelectionChangedHandle);
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->OnPreSave().RemoveAll(this);
		Sequencer->OnActivateSequence().RemoveAll(this);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandle);
	FCoreUObjectDelegates::OnObjectPreSave.Remove(OnObjectSavedHandle);
#endif
}


/* FDaySequenceSpawnRegister interface
 *****************************************************************************/

UObject* FDaySequenceEditorSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	TGuardValue<bool> Guard(bShouldClearSelectionCache, false);

	UObject* NewObject = FDaySequenceSpawnRegister::SpawnObject(Spawnable, TemplateID, SharedPlaybackState);
	
	if (AActor* NewActor = Cast<AActor>(NewObject))
	{
		// Mark as replay rewindable so it persists while a replay is going on in PIE.
		NewActor->bReplayRewindable = true;

		// Add an entry to the tracked objects map to keep track of this object (so that it can be saved when modified)
		TrackedObjects.Add(NewActor, FTrackedObjectState(TemplateID, Spawnable.GetGuid()));

		// Select the actor if we think it should be selected
		if (SelectedSpawnedObjects.Contains(FMovieSceneSpawnRegisterKey(TemplateID, Spawnable.GetGuid(), 0)))
		{
			GEditor->SelectActor(NewActor, true /*bSelected*/, true /*bNotify*/);
		}
	}

	return NewObject;
}


void FDaySequenceEditorSpawnRegister::PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID)
{
	TGuardValue<bool> Guard(bShouldClearSelectionCache, false);

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	UMovieSceneSequence*  Sequence      = Sequencer.IsValid() ? Sequencer->GetEvaluationTemplate().GetSequence(TemplateID) : nullptr;
	FMovieSceneSpawnable* Spawnable     = Sequence && Sequence->GetMovieScene() ? Sequence->GetMovieScene()->FindSpawnable(BindingId) : nullptr;
	UObject*              SpawnedObject = FindSpawnedObject(BindingId, TemplateID, BindingIndex).Get();

	if (SpawnedObject && Spawnable)
	{
		const FTrackedObjectState* TrackedState = TrackedObjects.Find(&Object);
		if (TrackedState && TrackedState->bHasBeenModified)
		{
			// SaveDefaultSpawnableState will reset bHasBeenModified to false
			SaveDefaultSpawnableStateImpl(*Spawnable, Sequence, SpawnedObject, Sequencer->GetSharedPlaybackState());

			Sequence->MarkPackageDirty();
		}
	}

	// Cache its selection state
	AActor* Actor = Cast<AActor>(&Object);
	if (Actor && GEditor->GetSelectedActors()->IsSelected(Actor))
	{
		SelectedSpawnedObjects.Add(FMovieSceneSpawnRegisterKey(TemplateID, BindingId, 0));
		GEditor->SelectActor(Actor, false /*bSelected*/, true /*bNotify*/);
	}

	FObjectKey ThisObject(&Object);
	TrackedObjects.Remove(ThisObject);

	FDaySequenceSpawnRegister::PreDestroyObject(Object, BindingId, BindingIndex, TemplateID);
}

#if WITH_EDITOR
void FDaySequenceEditorSpawnRegister::SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	SaveDefaultSpawnableState(Spawnable.GetGuid(), 0, TemplateID, SharedPlaybackState);
}

void FDaySequenceEditorSpawnRegister::SaveDefaultSpawnableState(const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(TemplateID);

	UObject* Object = FindSpawnedObject(BindingId, TemplateID, BindingIndex).Get();
	if (Object && Sequence)
	{
		if (FMovieSceneSpawnable* Spawnable = Sequence->GetMovieScene()->FindSpawnable(BindingId))
		{
			SaveDefaultSpawnableStateImpl(*Spawnable, Sequence, Object, SharedPlaybackState);
			Sequence->MarkPackageDirty();
		}
	}
}
#endif

void FDaySequenceEditorSpawnRegister::SaveDefaultSpawnableStateImpl(FMovieSceneSpawnable& Spawnable, UMovieSceneSequence* Sequence, UObject* SpawnedObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	IMovieScenePlayer* Player = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

	FMovieSceneAnimTypeID SpawnablesTypeID = UMovieSceneSpawnablesSystem::GetAnimTypeID();
	auto RestorePredicate = [SpawnablesTypeID](FMovieSceneAnimTypeID TypeID){ return TypeID != SpawnablesTypeID; };

	if (AActor* Actor = Cast<AActor>(SpawnedObject))
	{
		// Restore state on any components
		for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(Actor))
		{
			if (Component)
			{
				Player->PreAnimatedState.RestorePreAnimatedState(*Component, RestorePredicate);
			}
		}
	}

	// Restore state on the object itself
	Player->PreAnimatedState.RestorePreAnimatedState(*SpawnedObject, RestorePredicate);

	// Copy the template
	Spawnable.CopyObjectTemplate(*SpawnedObject, *Sequence);

	if (FTrackedObjectState* TrackedState = TrackedObjects.Find(SpawnedObject))
	{
		TrackedState->bHasBeenModified = false;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->RequestInvalidateCachedData();
		Sequencer->RequestEvaluate();
	}
}

/* FDaySequenceEditorSpawnRegister implementation
 *****************************************************************************/

void FDaySequenceEditorSpawnRegister::SetSequencer(const TSharedPtr<ISequencer>& Sequencer)
{
	WeakSequencer = Sequencer;
}


/* FDaySequenceEditorSpawnRegister callbacks
 *****************************************************************************/

void FDaySequenceEditorSpawnRegister::HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (bShouldClearSelectionCache)
	{
		SelectedSpawnedObjects.Reset();
	}
}

void FDaySequenceEditorSpawnRegister::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	for (auto& Pair : Register)
	{
		TWeakObjectPtr<>& WeakObject = Pair.Value.Object;
		UObject* SpawnedObject = WeakObject.Get();
		if (UObject* NewObject = OldToNewInstanceMap.FindRef(SpawnedObject))
		{
			// Reassign the object
			WeakObject = NewObject;

			// It's a spawnable, so ensure it's transient
			NewObject->SetFlags(RF_Transient);

			// Invalidate the binding - it will be resolved if it's ever asked for again
			Sequencer->GetEvaluationState()->Invalidate(Pair.Key.BindingId, Pair.Key.TemplateID);
		}
	}
}

void FDaySequenceEditorSpawnRegister::OnObjectModified(UObject* ModifiedObject)
{
	FTrackedObjectState* TrackedState = TrackedObjects.Find(ModifiedObject);
	while (!TrackedState && ModifiedObject)
	{
		TrackedState = TrackedObjects.Find(ModifiedObject);
		ModifiedObject = ModifiedObject->GetOuter();
	}

	if (TrackedState)
	{
		TrackedState->bHasBeenModified = true;

		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		UMovieSceneSequence*   OwningSequence = Sequencer.IsValid() ? Sequencer->GetEvaluationTemplate().GetSequence(TrackedState->TemplateID) : nullptr;
		if (OwningSequence)
		{
			OwningSequence->MarkPackageDirty();
			SequencesWithModifiedObjects.Add(OwningSequence);
		}
	}
}

void FDaySequenceEditorSpawnRegister::OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext)
{
	UMovieSceneSequence* SequenceBeingSaved = Cast<UMovieSceneSequence>(Object);
	if (SequenceBeingSaved && SequencesWithModifiedObjects.Contains(SequenceBeingSaved))
	{
		UMovieScene* MovieSceneBeingSaved = SequenceBeingSaved->GetMovieScene();

		// The object being saved is a movie scene sequence that we've tracked as having a modified spawnable in the world.
		// We need to go through all templates in the current sequence that reference this sequence, saving default state
		// for any spawned objects that have been modified.
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

		if (Sequencer.IsValid())
		{
			for (const TTuple<FObjectKey, FTrackedObjectState>& Pair : TrackedObjects)
			{
				UObject* SpawnedObject = Pair.Key.ResolveObjectPtr();
				UMovieSceneSequence*  ThisSequence = Sequencer->GetEvaluationTemplate().GetSequence(Pair.Value.TemplateID);
				FMovieSceneSpawnable* Spawnable    = MovieSceneBeingSaved->FindSpawnable(Pair.Value.ObjectBindingID);

				if (SpawnedObject && Spawnable && ThisSequence == SequenceBeingSaved)
				{
					SaveDefaultSpawnableStateImpl(*Spawnable, ThisSequence, SpawnedObject, Sequencer->GetSharedPlaybackState());
				}
			}
		}
	}
}

#if WITH_EDITOR

TValueOrError<FNewSpawnable, FText> FDaySequenceEditorSpawnRegister::CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory)
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		TValueOrError<FNewSpawnable, FText> Result = MovieSceneObjectSpawner->CreateNewSpawnableType(SourceObject, OwnerMovieScene, ActorFactory);
		if (Result.IsValid())
		{
			return Result;
		}
	}

	return MakeError(LOCTEXT("NoSpawnerFound", "No spawner found to create new spawnable type"));
}

void FDaySequenceEditorSpawnRegister::SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings)
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (MovieSceneObjectSpawner->CanSetupDefaultsForSpawnable(SpawnedObject))
		{
			MovieSceneObjectSpawner->SetupDefaultsForSpawnable(SpawnedObject, Guid, TransformData, Sequencer, Settings);
			return;
		}
	}
}

void FDaySequenceEditorSpawnRegister::HandleConvertPossessableToSpawnable(UObject* OldObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TOptional<FTransformData>& OutTransformData)
{
	// @TODO: this could probably be handed off to a spawner if we need anything else to be convertible between spawnable/posessable

	AActor* OldActor = Cast<AActor>(OldObject);
	if (OldActor)
	{
		if (OldActor->GetRootComponent())
		{
			OutTransformData.Emplace();
			OutTransformData->Translation = OldActor->GetRootComponent()->GetRelativeLocation();
			OutTransformData->Rotation = OldActor->GetRootComponent()->GetRelativeRotation();
			OutTransformData->Scale = OldActor->GetRootComponent()->GetRelativeScale3D();
		}

		GEditor->SelectActor(OldActor, false, true);
		UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if (World)
		{
			World->EditorDestroyActor(OldActor, true);

			GEditor->BroadcastLevelActorListChanged();
		}
	}
}

bool FDaySequenceEditorSpawnRegister::CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		UObject* ObjectTemplate = Spawnable.GetObjectTemplate();

		if (ObjectTemplate && ObjectTemplate->IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			return MovieSceneObjectSpawner->CanConvertSpawnableToPossessable(Spawnable);
		}
	}

	return false;
}

#endif


#undef LOCTEXT_NAMESPACE
