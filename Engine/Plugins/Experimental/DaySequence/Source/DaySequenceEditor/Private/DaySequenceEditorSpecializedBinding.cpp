// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorSpecializedBinding.h"

#include "DaySequence.h"
#include "ISequencer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "DaySequenceEditorSpecializedBinding"

FDaySequenceEditorSpecializedBinding::FDaySequenceEditorSpecializedBinding(TSharedRef<ISequencer> InSequencer)
	: Sequencer(InSequencer)
{
}

FText FDaySequenceEditorSpecializedBinding::GetDisplayName() const
{
	return LOCTEXT("DaySequenceEditorSpecializedBinding_DisplayName", "Day Sequence Specialized Binding");
}

void FDaySequenceEditorSpecializedBinding::BuildSequencerAddMenu(FMenuBuilder& MenuBuilder)
{
	// todo [nickolas.drake]: add icon
	MenuBuilder.AddSubMenu(
		LOCTEXT("SpecializedBindingLabel", "Specialized Bindings"),
		LOCTEXT("SpecializedBindingToolTip", "Add specialized binding types which have special resolution rules"),
		FNewMenuDelegate::CreateRaw(this, &FDaySequenceEditorSpecializedBinding::AddSpecializedBindingMenuExtensions)
	);
}

bool FDaySequenceEditorSpecializedBinding::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return InSequence->GetClass() == UDaySequence::StaticClass();
}

void FDaySequenceEditorSpecializedBinding::AddSpecializedBindingMenuExtensions(FMenuBuilder& MenuBuilder)
{
	// generates an FUIAction for a binding specialization type
	auto GetAddBindingAction = [WeakSequencer = Sequencer](EDaySequenceBindingReferenceSpecialization Specialization)
	{
		FUIAction Action;
	
		Action.ExecuteAction = FExecuteAction::CreateLambda([WeakSequencer, Specialization]()
		{
			if (const TSharedPtr<ISequencer> PinnedSequencer = WeakSequencer.Pin())
			{
				if (UDaySequence* FocusedSequence = Cast<UDaySequence>(PinnedSequencer->GetFocusedMovieSceneSequence()))
				{
					return FocusedSequence->AddSpecializedBinding(Specialization);
				}
			}
		});

		Action.CanExecuteAction = FCanExecuteAction::CreateLambda([WeakSequencer, Specialization]()
		{
			if (const TSharedPtr<ISequencer> PinnedSequencer = WeakSequencer.Pin())
			{
				if (const UDaySequence* FocusedSequence = Cast<UDaySequence>(PinnedSequencer->GetFocusedMovieSceneSequence()))
				{
					// only allow creation if binding type isn't present already
					return !FocusedSequence->GetSpecializedBinding(Specialization).IsValid();
				}
			}
		
			return false;
		});

		return Action;
	};

	// Add an entry for a root binding
	MenuBuilder.AddMenuEntry(LOCTEXT("RootActorBindingLabel", "Root Day Sequence Actor Binding"),
		LOCTEXT("RootActorBindingTooltip", "Add a new root actor binding. This allows a sequence to animate a generic day sequence actor."),
		FSlateIcon(),
		GetAddBindingAction(EDaySequenceBindingReferenceSpecialization::Root)
	);
	
	// Add an entry for a camera modifier binding
	MenuBuilder.AddMenuEntry(LOCTEXT("CameraModifierBindingLabel", "Camera Modifier Binding"),
		LOCTEXT("CameraModifierBindingTooltip", "Add a new camera modifier binding. This allows a sequence to animate post process effects for local players."),
		FSlateIcon(),
		GetAddBindingAction(EDaySequenceBindingReferenceSpecialization::CameraModifier)
	);
}

#undef LOCTEXT_NAMESPACE
