// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerEditorObjectBinding.h"
#include "Templates/SharedPointer.h"

class ISequencer;
class AActor;

class FDaySequenceEditorSpecializedBinding : public ISequencerEditorObjectBinding
{
public:

	FDaySequenceEditorSpecializedBinding(TSharedRef<ISequencer> InSequencer);

	// ISequencerEditorObjectBinding interface
	virtual FText GetDisplayName() const override;
	virtual void BuildSequencerAddMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

private:

	/** Menu extension callback for the specialized binding menu */
	void AddSpecializedBindingMenuExtensions(FMenuBuilder& MenuBuilder);

private:
	TWeakPtr<ISequencer> Sequencer;
};
