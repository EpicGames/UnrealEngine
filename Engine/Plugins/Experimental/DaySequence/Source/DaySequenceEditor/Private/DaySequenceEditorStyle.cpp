// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Trace/Detail/LogScope.inl"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FDaySequenceEditorStyle::StyleInstance = nullptr;

void FDaySequenceEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FDaySequenceEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FDaySequenceEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("DaySequenceStyle"));
	return StyleSetName;
}


const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);

TSharedRef< FSlateStyleSet > FDaySequenceEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet("DaySequenceStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("DaySequence")->GetBaseDir() / TEXT("Resources"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir());

	Style->Set("DaySequenceEditor.OpenDaySequenceEditor", new IMAGE_BRUSH_SVG("LightBulb", Icon20x20));
	Style->Set("DaySequenceEditor.PossessNewActor", new FSlateImageBrush(Style->RootToCoreContentDir("Editor/Slate/Sequencer/Dropdown_icons/Icon_Actor_To_Sequencer_16x.png"), Icon16x16));
	Style->Set("DaySequenceEditor.PossessNewActor.Small", new FSlateImageBrush(Style->RootToCoreContentDir("Editor/Slate/Sequencer/Dropdown_icons/Icon_Actor_To_Sequencer_16x.png"), Icon16x16));
	Style->Set("DaySequenceEditor.ViewportToolBar", new FSlateVectorImageBrush(Style->RootToCoreContentDir("Editor/Slate/Starship/Common/Atmosphere.svg"), Icon16x16));
	Style->Set("ClassIcon.DaySequenceActor", new FSlateVectorImageBrush(RootToContentDir("DayNightCycle", TEXT(".svg")), Icon16x16, FSlateColor(EStyleColor::Foreground)));
	Style->Set("ClassThumbnail.DaySequenceActor", new FSlateVectorImageBrush(RootToContentDir("DayNightCycle", TEXT(".svg")), Icon64x64, FSlateColor(EStyleColor::Foreground)));

	return Style;
}


void FDaySequenceEditorStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

TSharedRef<ISlateStyle> FDaySequenceEditorStyle::Get()
{
	return StyleInstance.ToSharedRef();
}
