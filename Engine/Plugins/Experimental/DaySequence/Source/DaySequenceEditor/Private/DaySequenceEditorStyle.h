// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISlateStyle;

class FDaySequenceEditorStyle
{
public:
	static void Initialize();
	static void Shutdown();

	/** Reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the Shooter game */
	static TSharedRef<ISlateStyle> Get();

	static FName GetStyleSetName();

private:
	static TSharedRef<class FSlateStyleSet> Create();

private:
	static TSharedPtr<class FSlateStyleSet> StyleInstance;
};
