// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceEditorToolkit.h"
#include "DaySequence.h"
#include "DaySequenceEditorMenuContext.h"
#include "DaySequenceEditorSettings.h"
#include "DaySequenceEditorSpawnRegister.h"
#include "DaySequencePlaybackContext.h"
#include "DaySequenceSubsystem.h"
#include "DaySequenceActor.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Containers/ArrayBuilder.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "KeyParams.h"
#include "KeyPropertyParams.h"
#include "LevelEditor.h"
#include "LevelEditorSequencerIntegration.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "ToolMenuSection.h"
#include "UnrealEdMisc.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/UnrealType.h"
#include "Widgets/Docking/SDockTab.h"

// @todo sequencer: hack: setting defaults for transform tracks
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"


#define LOCTEXT_NAMESPACE "DaySequenceEditor"


/* Local constants
 *****************************************************************************/

const FName FDaySequenceEditorToolkit::SequencerMainTabId(TEXT("Sequencer_SequencerMain"));

namespace SequencerDefs
{
	static const FName SequencerAppIdentifier(TEXT("SequencerApp"));
}

static TArray<FDaySequenceEditorToolkit*> OpenToolkits;

void FDaySequenceEditorToolkit::IterateOpenToolkits(TFunctionRef<bool(FDaySequenceEditorToolkit&)> Iter)
{
	for (FDaySequenceEditorToolkit* Toolkit : OpenToolkits)
	{
		if (!Iter(*Toolkit))
		{
			return;
		}
	}
}

void FDaySequenceEditorToolkit::CloseOpenToolkits(TFunctionRef<bool(FDaySequenceEditorToolkit&)> Iter)
{
	for (int Idx = OpenToolkits.Num()-1; Idx >= 0; --Idx)
	{
		FDaySequenceEditorToolkit* Toolkit = OpenToolkits[Idx];
		if (Toolkit && Iter(*Toolkit))
		{
			Toolkit->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
		}
	}
}

bool FDaySequenceEditorToolkit::HasOpenSequenceEditorToolkits()
{
	bool bResult = false;
	IterateOpenToolkits([&bResult](FDaySequenceEditorToolkit& Toolkit)
	{
		bResult = !Toolkit.IsActorPreview();
		return !bResult;
	});
	return bResult;
}

bool FDaySequenceEditorToolkit::HasOpenActorPreviewToolkits()
{
	bool bResult = false;
	IterateOpenToolkits([&bResult](FDaySequenceEditorToolkit& Toolkit)
	{
		bResult = Toolkit.IsActorPreview();
		return !bResult;
	});
	return bResult;
}

FDaySequenceEditorToolkit::FDaySequenceEditorToolkitOpened& FDaySequenceEditorToolkit::OnOpened()
{
	static FDaySequenceEditorToolkitOpened OnOpenedEvent;
	return OnOpenedEvent;
}

FDaySequenceEditorToolkit::FDaySequenceEditorToolkitClosed& FDaySequenceEditorToolkit::OnClosed()
{
	static FDaySequenceEditorToolkitClosed OnClosedEvent;
	return OnClosedEvent;
}

FDaySequenceEditorToolkit::FDaySequenceEditorToolkitDestroyed& FDaySequenceEditorToolkit::OnDestroyed()
{
	static FDaySequenceEditorToolkitDestroyed OnDestroyedEvent;
	return OnDestroyedEvent;
}

FDaySequenceEditorToolkit::FDaySequenceEditorToolkitPostMapChanged& FDaySequenceEditorToolkit::OnToolkitPostMapChanged()
{
	static FDaySequenceEditorToolkitPostMapChanged OnPostMapChangedEvent;
	return OnPostMapChangedEvent;
}

/* FDaySequenceEditorToolkit structors
 *****************************************************************************/

FDaySequenceEditorToolkit::FDaySequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: Style(InStyle)
{
	OpenToolkits.Add(this);
}


FDaySequenceEditorToolkit::~FDaySequenceEditorToolkit()
{
	OpenToolkits.Remove(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		// Actor preview toolkits do not attach UI to the level editor, so do not clear
		// the attached sequencer on the level editor module. Doing so has the side effect
		// of invoking the empty Sequencer tab which makes it both visible and in front.
		if (!IsActorPreview())
		{
			LevelEditorModule.AttachSequencer(SNullWidget::NullWidget, nullptr);
		}
		FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());

		// unregister delegates
		LevelEditorModule.OnMapChanged().RemoveAll(this);
	}

	Sequencer->Close();

	// If this toolkit opened the root sequence of a DaySequenceActor, edits are only supported
	// to the subsequences. Regenerate the root sequence to ensure that any unsupported modifications
	// are cleared.
	if (RootActor)
	{
		// When reinstancing actors due to BP recompile for example, the process closes all asset
		// editors in advance of the reinstance to avoid asset editors referencing stale data.
		// Some systems like DaySequenceActorPreview listen to Pre/PostRootSequenceChange to reapply
		// the preview toolkit. We run the regeneration of the root sequence on tick to avoid this
		// case.
		RootActor->bForceDisableDayInterpCurve = false;
		RootActor->UpdateRootSequenceOnTick(ADaySequenceActor::EUpdateRootSequenceMode::Reinitialize);
	}

	OnDestroyed().Broadcast(*this);
}


/* FDaySequenceEditorToolkit interface
 *****************************************************************************/

void FDaySequenceEditorToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UDaySequence* InDaySequence)
{
	if (!InitToolkitHost.IsValid() || !InDaySequence)
	{
		return;
	}
	
	// There can only be one toolkit active at a time. Close all others now.
	CloseOpenToolkits([this](FDaySequenceEditorToolkit& Toolkit)
	{
		return &Toolkit != this;
	});

	DaySequence = InDaySequence;
	RootActor = Cast<ADaySequenceActor>(InDaySequence->GetOuter());
	
	if (RootActor)
	{
		RootActor->bForceDisableDayInterpCurve = true;
	}
	
	PlaybackContext = MakeShared<FDaySequencePlaybackContext>(InDaySequence);
	TSharedRef<FDaySequenceEditorSpawnRegister> SpawnRegister = MakeShareable(new FDaySequenceEditorSpawnRegister);
	
	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.RootSequence = DaySequence;
		SequencerInitParams.bEditWithinLevelEditor = true;
		SequencerInitParams.ToolkitHost = InitToolkitHost;
		SequencerInitParams.SpawnRegister = SpawnRegister;

		SequencerInitParams.PlaybackContext.Bind(PlaybackContext.ToSharedRef(), &FDaySequencePlaybackContext::GetPlaybackContextAsObject);
		SequencerInitParams.PlaybackClient.Bind(PlaybackContext.ToSharedRef(), &FDaySequencePlaybackContext::GetPlaybackClientAsInterface);

		SequencerInitParams.ViewParams.UniqueName = "DaySequenceEditor";
		SequencerInitParams.ViewParams.ScrubberStyle = ESequencerScrubberStyle::FrameBlock;
		SequencerInitParams.ViewParams.OnReceivedFocus.BindRaw(this, &FDaySequenceEditorToolkit::OnSequencerReceivedFocus);
		SequencerInitParams.ViewParams.OnInitToolMenuContext.BindRaw(this, &FDaySequenceEditorToolkit::OnInitToolMenuContext);
		SequencerInitParams.ViewParams.bReadOnly = false;
		
		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
		SequencerInitParams.HostCapabilities.bSupportsSaveMovieSceneAsset = true;
		SequencerInitParams.HostCapabilities.bSupportsRecording = true;
		SequencerInitParams.HostCapabilities.bSupportsRenderMovie = true;
	}
	
	InitializeInternal(Mode, InitToolkitHost, SequencerInitParams, SpawnRegister);
}

void FDaySequenceEditorToolkit::InitializeActorPreview(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ADaySequenceActor* InDayActor)
{
	if (!InitToolkitHost.IsValid())
	{
		return;
	}
	
	PreviewActor = InDayActor;
	
	// There can only be one toolkit active at a time. Close all others now.
	CloseOpenToolkits([this](const FDaySequenceEditorToolkit& Toolkit)
	{
		return &Toolkit != this;
	});
	
	UDaySequence* InDaySequence = InDayActor->GetRootSequence();
	
	DaySequence = InDaySequence;
	PlaybackContext = MakeShared<FDaySequencePlaybackContext>(InDaySequence);
	TSharedRef<FDaySequenceEditorSpawnRegister> SpawnRegister = MakeShareable(new FDaySequenceEditorSpawnRegister);
	
	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.RootSequence = DaySequence;
		SequencerInitParams.bEditWithinLevelEditor = true;
		SequencerInitParams.ToolkitHost = InitToolkitHost;
		SequencerInitParams.SpawnRegister = SpawnRegister;

		SequencerInitParams.PlaybackContext.Bind(PlaybackContext.ToSharedRef(), &FDaySequencePlaybackContext::GetPlaybackContextAsObject);
		SequencerInitParams.PlaybackClient.Bind(PlaybackContext.ToSharedRef(), &FDaySequencePlaybackContext::GetPlaybackClientAsInterface);

		SequencerInitParams.ViewParams.UniqueName = "DaySequenceEditor";
		SequencerInitParams.ViewParams.ScrubberStyle = ESequencerScrubberStyle::FrameBlock;
		SequencerInitParams.ViewParams.OnReceivedFocus.BindRaw(this, &FDaySequenceEditorToolkit::OnSequencerReceivedFocus);
		SequencerInitParams.ViewParams.OnInitToolMenuContext.BindRaw(this, &FDaySequenceEditorToolkit::OnInitToolMenuContext);
		SequencerInitParams.ViewParams.bReadOnly = true;
		
		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = false;
		SequencerInitParams.HostCapabilities.bSupportsSaveMovieSceneAsset = false;
		SequencerInitParams.HostCapabilities.bSupportsRecording = false;
		SequencerInitParams.HostCapabilities.bSupportsRenderMovie = false;
	}
	
	InitializeInternal(Mode, InitToolkitHost, SequencerInitParams, SpawnRegister);
}

void FDaySequenceEditorToolkit::InitializeInternal(
	const EToolkitMode::Type Mode,
	const TSharedPtr<IToolkitHost>& InitToolkitHost,
	const FSequencerInitParams& SequencerInitParams,
	TSharedRef<FDaySequenceEditorSpawnRegister>& SpawnRegister)
{
	// create tab layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_DaySequenceEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->Split
				(
					FTabManager::NewStack()
						->AddTab(SequencerMainTabId, ETabState::OpenedTab)
				)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = false;

	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, SequencerDefs::SequencerAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, DaySequence);

	ExtendSequencerToolbar("Sequencer.MainToolBar");

	// initialize sequencer
	Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
	SpawnRegister->SetSequencer(Sequencer);
	Sequencer->OnActorAddedToSequencer().AddSP(this, &FDaySequenceEditorToolkit::HandleActorAddedToSequencer);

	// Set appropriate default playback speed based on the ratio of the root sequence duration to time per cycle.
	if (const ADaySequenceActor* DayActor = RootActor ? RootActor : PreviewActor)
	{
		const UMovieScene* MovieScene = SequencerInitParams.RootSequence->GetMovieScene();
		const float SequenceDurationHours = MovieScene->GetTickResolution().AsSeconds(MovieScene->GetPlaybackRange().GetUpperBoundValue()) / 3600.f;
		const float DesiredDurationHours = DayActor->GetTimeOfDay();
		Sequencer->SetPlaybackSpeed(SequenceDurationHours / DesiredDurationHours);
	}
	
	const bool bIsSequenceEditor = !IsActorPreview();

	Sequencer->OnGlobalTimeChanged().AddSP(this, &FDaySequenceEditorToolkit::OnGlobalTimeChanged);
	
	FLevelEditorSequencerIntegrationOptions Options;
	Options.bRequiresLevelEvents = true;
	Options.bRequiresActorEvents = true;
	Options.bForceRefreshDetails = bIsSequenceEditor;
	Options.bAttachOutlinerColumns = bIsSequenceEditor;
	Options.bActivateSequencerEdMode = bIsSequenceEditor;
	Options.bSyncBindingsToActorLabels = bIsSequenceEditor;

	FLevelEditorSequencerIntegration::Get().AddSequencer(Sequencer.ToSharedRef(), Options);

	// @todo remove when world-centric mode is added
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	if (bIsSequenceEditor)
	{
		// Now Attach so this window will appear in the correct front first order
		TSharedPtr<SDockTab> DockTab = LevelEditorModule.AttachSequencer(Sequencer->GetSequencerWidget(), SharedThis(this));
		if (DockTab.IsValid())
		{
			TAttribute<FText> LabelSuffix = TAttribute<FText>(this, &FDaySequenceEditorToolkit::GetTabSuffix);
			DockTab->SetTabLabelSuffix(LabelSuffix);
		}

		if (RootActor)
		{
			RootActor->GetOnPostRootSequenceChanged().AddSPLambda(this, [this]()
			{
				if (IsValid(RootActor))
				{
					if (UDaySequence* CurrentDaySequence = RootActor->GetRootSequence(); CurrentDaySequence && CurrentDaySequence != DaySequence)
					{
						Sequencer->ResetToNewRootSequence(*CurrentDaySequence);
						DaySequence = CurrentDaySequence;
					}
				}
			});
		}
	}

	// We need to find out when the user loads a new map, because we might need to re-create puppet actors
	// when previewing a MovieScene
	LevelEditorModule.OnMapChanged().AddRaw(this, &FDaySequenceEditorToolkit::HandleMapChanged);

	FDaySequenceEditorToolkit::OnOpened().Broadcast(*this);
}


/* IToolkit interface
 *****************************************************************************/

FText FDaySequenceEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Day Sequence Editor");
}


FName FDaySequenceEditorToolkit::GetToolkitFName() const
{
	static FName SequencerName("DaySequenceEditor");
	return SequencerName;
}


FLinearColor FDaySequenceEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}


FString FDaySequenceEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Sequencer ").ToString();
}


FText FDaySequenceEditorToolkit::GetTabSuffix() const
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();

	if (Sequence == nullptr)
	{
		return FText::GetEmpty();
	}
	
	const bool bIsDirty = Sequence->GetMovieScene()->GetOuter()->GetOutermost()->IsDirty();
	if (bIsDirty)
	{
		return LOCTEXT("TabSuffixAsterix", "*");
	}

	return FText::GetEmpty();
}


/* FDaySequenceEditorToolkit implementation
 *****************************************************************************/

void FDaySequenceEditorToolkit::ExtendSequencerToolbar(FName InToolMenuName)
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(InToolMenuName);

	const FToolMenuInsert SectionInsertLocation("BaseCommands", EToolMenuInsertType::Before);

	{
		ToolMenu->AddDynamicSection("DaySequenceEditorDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{	
			UDaySequenceEditorMenuContext* DaySequenceEditorMenuContext = InMenu->FindContext<UDaySequenceEditorMenuContext>();
			if (DaySequenceEditorMenuContext && DaySequenceEditorMenuContext->Toolkit.IsValid())
			{
				const FName SequencerToolbarStyleName = "SequencerToolbar";
			
				FToolMenuEntry PlaybackContextEntry = FToolMenuEntry::InitWidget(
					"PlaybackContext",
					DaySequenceEditorMenuContext->Toolkit.Pin()->PlaybackContext->BuildWorldPickerCombo(),
					LOCTEXT("PlaybackContext", "PlaybackContext")
				);
				PlaybackContextEntry.StyleNameOverride = SequencerToolbarStyleName;

				FToolMenuSection& Section = InMenu->AddSection("DaySequenceEditor");
				Section.AddEntry(PlaybackContextEntry);
			}
		}), SectionInsertLocation);
	}
}

void FDaySequenceEditorToolkit::AddDefaultTracksForActor(AActor& Actor, const FGuid Binding)
{
	// get focused movie scene
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();

	if (Sequence == nullptr)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (MovieScene == nullptr)
	{
		return;
	}

	// Create a default section for a new track.
	//
	// @param InNewTrack the track to create a default section for
	// @param InComponent for MovieScene3DTransformTrack, optional scene component to initialize the relative transform
	auto CreateDefaultTrackSection = [this, &Actor](UMovieSceneTrack* InNewTrack, UObject* InComponent)
	{
		// Track class permissions can deny track creation. (UMovieScene::IsTrackClassAllowed)
		if (!InNewTrack)
		{
			return;
		}

#if WITH_EDITORONLY_DATA
		if (!InNewTrack->SupportsDefaultSections())
		{
			return;
		}
#endif
		
		UMovieSceneSection* NewSection;
		if (InNewTrack->GetAllSections().Num() > 0)
		{
			NewSection = InNewTrack->GetAllSections()[0];
		}
		else
		{
			NewSection = InNewTrack->CreateNewSection();
			InNewTrack->AddSection(*NewSection);
		}

		// @todo sequencer: hack: setting defaults for transform tracks
		if (InNewTrack->IsA(UMovieScene3DTransformTrack::StaticClass()) && Sequencer->GetAutoSetTrackDefaults())
		{
			auto TransformSection = Cast<UMovieScene3DTransformSection>(NewSection);

			FVector Location = Actor.GetActorLocation();
			FRotator Rotation = Actor.GetActorRotation();
			FVector Scale = Actor.GetActorScale();

			if (USceneComponent* SceneComponent = Cast<USceneComponent>(InComponent))
			{
				FTransform ActorRelativeTransform = SceneComponent->GetRelativeTransform();

				Location = ActorRelativeTransform.GetTranslation();
				Rotation = ActorRelativeTransform.GetRotation().Rotator();
				Scale = ActorRelativeTransform.GetScale3D();
			}

			TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			DoubleChannels[0]->SetDefault(Location.X);
			DoubleChannels[1]->SetDefault(Location.Y);
			DoubleChannels[2]->SetDefault(Location.Z);

			DoubleChannels[3]->SetDefault(Rotation.Euler().X);
			DoubleChannels[4]->SetDefault(Rotation.Euler().Y);
			DoubleChannels[5]->SetDefault(Rotation.Euler().Z);

			DoubleChannels[6]->SetDefault(Scale.X);
			DoubleChannels[7]->SetDefault(Scale.Y);
			DoubleChannels[8]->SetDefault(Scale.Z);
		}

		if (GetSequencer()->GetInfiniteKeyAreas())
		{
			NewSection->SetRange(TRange<FFrameNumber>::All());
		}
	};

	// add default tracks
	for (const FDaySequenceTrackSettings& TrackSettings : GetDefault<UDaySequenceEditorSettings>()->TrackSettings)
	{
		UClass* MatchingActorClass = TrackSettings.MatchingActorClass.ResolveClass();

		if ((MatchingActorClass == nullptr) || !Actor.IsA(MatchingActorClass))
		{
			continue;
		}

		// add tracks by type
		for (const FSoftClassPath& DefaultTrack : TrackSettings.DefaultTracks)
		{
			UClass* TrackClass = DefaultTrack.ResolveClass();

			// exclude any tracks explicitly marked for exclusion
			for (const FDaySequenceTrackSettings& ExcludeTrackSettings : GetDefault<UDaySequenceEditorSettings>()->TrackSettings)
			{
				UClass* ExcludeMatchingActorClass = ExcludeTrackSettings.MatchingActorClass.ResolveClass();

				if ((ExcludeMatchingActorClass == nullptr) || !Actor.IsA(ExcludeMatchingActorClass))
				{
					continue;
				}
				
				for (const FSoftClassPath& ExcludeDefaultTrack : ExcludeTrackSettings.ExcludeDefaultTracks)
				{
					if (ExcludeDefaultTrack == DefaultTrack)
					{
						TrackClass = nullptr;
						break;
					}
				}				
			}

			if (TrackClass != nullptr)
			{
				UMovieSceneTrack* NewTrack = MovieScene->FindTrack(TrackClass, Binding);
				if (!NewTrack)
				{
					NewTrack = MovieScene->AddTrack(TrackClass, Binding);
				}
				CreateDefaultTrackSection(NewTrack, Actor.GetRootComponent());
			}
		}

		// construct a map of the properties that should be excluded per component
		TMap<UObject*, TArray<FString> > ExcludePropertyTracksMap;
		for (const FDaySequenceTrackSettings& ExcludeTrackSettings : GetDefault<UDaySequenceEditorSettings>()->TrackSettings)
		{
			UClass* ExcludeMatchingActorClass = ExcludeTrackSettings.MatchingActorClass.ResolveClass();

			if ((ExcludeMatchingActorClass == nullptr) || !Actor.IsA(ExcludeMatchingActorClass))
			{
				continue;
			}

			for (const FDaySequencePropertyTrackSettings& PropertyTrackSettings : ExcludeTrackSettings.ExcludeDefaultPropertyTracks)
			{
				UObject* PropertyOwner = &Actor;

				// determine object hierarchy
				TArray<FString> ComponentNames;
				PropertyTrackSettings.ComponentPath.ParseIntoArray(ComponentNames, TEXT("."));

				for (const FString& ComponentName : ComponentNames)
				{
					PropertyOwner = FindObjectFast<UObject>(PropertyOwner, *ComponentName);

					if (PropertyOwner == nullptr)
					{
						continue;
					}
				}

				if (PropertyOwner)
				{
					TArray<FString> PropertyNames;
					PropertyTrackSettings.PropertyPath.ParseIntoArray(PropertyNames, TEXT("."));

					ExcludePropertyTracksMap.Add(PropertyOwner, PropertyNames);
				}
			}
		}

		// add tracks by property
		for (const FDaySequencePropertyTrackSettings& PropertyTrackSettings : TrackSettings.DefaultPropertyTracks)
		{
			TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
			UObject* PropertyOwner = &Actor;

			// determine object hierarchy
			TArray<FString> ComponentNames;
			PropertyTrackSettings.ComponentPath.ParseIntoArray(ComponentNames, TEXT("."));

			for (const FString& ComponentName : ComponentNames)
			{
				PropertyOwner = FindObjectFast<UObject>(PropertyOwner, *ComponentName);

				if (PropertyOwner == nullptr)
				{
					return;
				}
			}

			UStruct* PropertyOwnerClass = PropertyOwner->GetClass();

			// determine property path
			TArray<FString> PropertyNames;
			PropertyTrackSettings.PropertyPath.ParseIntoArray(PropertyNames, TEXT("."));

			bool bReplaceWithTransformTrack = false;
			for (const FString& PropertyName : PropertyNames)
			{
				// skip past excluded properties
				if (ExcludePropertyTracksMap.Contains(PropertyOwner) && ExcludePropertyTracksMap[PropertyOwner].Contains(PropertyName))
				{
					PropertyPath = FPropertyPath::CreateEmpty();
					break;
				}

				FProperty* Property = PropertyOwnerClass->FindPropertyByName(*PropertyName);

				if (Property != nullptr)
				{
					PropertyPath->AddProperty(FPropertyInfo(Property));

					// Transform tracks are a special case and must be handled separately.
					if (PropertyOwner->IsA(USceneComponent::StaticClass()) &&
						(PropertyName == TEXT("RelativeLocation") || PropertyName == TEXT("RelativeRotation") || PropertyName == TEXT("RelativeScale3D")))
					{
						bReplaceWithTransformTrack = true;
						break;
					}
				}

				FStructProperty* StructProperty = CastField<FStructProperty>(Property);

				if (StructProperty != nullptr)
				{
					PropertyOwnerClass = StructProperty->Struct;
					continue;
				}

				FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);

				if (ObjectProperty != nullptr)
				{
					PropertyOwnerClass = ObjectProperty->PropertyClass;
					continue;
				}

				break;
			}
			
			if (bReplaceWithTransformTrack)
			{
				FGuid ComponentBinding = Sequencer->GetHandleToObject(PropertyOwner);
				UClass* TrackClass = UMovieScene3DTransformTrack::StaticClass();
				UMovieSceneTrack* NewTrack = MovieScene->FindTrack(TrackClass, ComponentBinding);
				if (!NewTrack)
				{
					NewTrack = MovieScene->AddTrack(TrackClass, ComponentBinding);
					CreateDefaultTrackSection(NewTrack, PropertyOwner);
				}
				continue;
			}

			if (!Sequencer->CanKeyProperty(FCanKeyPropertyParams(PropertyOwner->GetClass(), *PropertyPath)))
			{
				continue;
			}

			// key property
			FKeyPropertyParams KeyPropertyParams(TArrayBuilder<UObject*>().Add(PropertyOwner), *PropertyPath, ESequencerKeyMode::ManualKey);

			Sequencer->KeyProperty(KeyPropertyParams);
		}
	}
}


/* FDaySequenceEditorToolkit callbacks
 *****************************************************************************/

void FDaySequenceEditorToolkit::OnSequencerReceivedFocus()
{
	if (Sequencer.IsValid())
	{
		FLevelEditorSequencerIntegration::Get().OnSequencerReceivedFocus(Sequencer.ToSharedRef());
	}
}

void FDaySequenceEditorToolkit::OnInitToolMenuContext(FToolMenuContext& MenuContext)
{
	UDaySequenceEditorMenuContext* DaySequenceEditorMenuContext = NewObject<UDaySequenceEditorMenuContext>();
	DaySequenceEditorMenuContext->Toolkit = SharedThis(this);
	MenuContext.AddObject(DaySequenceEditorMenuContext);
}


void FDaySequenceEditorToolkit::HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding)
{
	AddDefaultTracksForActor(*Actor, Binding);
}

void FDaySequenceEditorToolkit::HandleMapChanged(class UWorld* NewWorld, EMapChangeType MapChangeType)
{
	// @todo sequencer: We should only wipe/respawn puppets that are affected by the world that is being changed! (multi-UWorld support)
	if( ( MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap || MapChangeType == EMapChangeType::TearDownWorld) )
	{
		Sequencer->GetSpawnRegister().CleanUp(*Sequencer);
		CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);

		OnToolkitPostMapChanged().Broadcast();
	}
}


TSharedRef<SDockTab> FDaySequenceEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (Args.GetTabId() == SequencerMainTabId)
	{
		TabWidget = Sequencer->GetSequencerWidget();
	}

	return SNew(SDockTab)
		.Label(LOCTEXT("SequencerMainTitle", "Sequencer"))
		.TabColorScale(GetTabColorScale())
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

void FDaySequenceEditorToolkit::OnClose()
{
	// Null out the DaySequence pointer to signify that this toolkit is no longer active.
	DaySequence = nullptr;
	OpenToolkits.Remove(this);
	OnClosed().Broadcast(*this);
}

bool FDaySequenceEditorToolkit::CanFindInContentBrowser() const
{
	// False so that sequencer doesn't take over Find In Content Browser functionality and always find the sequence asset
	return false;
}

void FDaySequenceEditorToolkit::OnGlobalTimeChanged()
{
	// If the sequence we are editing has a DSA outer (i.e. we are editing the root sequence), propagate sequencer time to actor.
	// If the sequence we are viewing is the PreviewActor, also propagate sequencer time to actor.
	if (ADaySequenceActor* DayActor = RootActor ? RootActor : PreviewActor)
	{
		// Convert sequencer time to equivalent game time.
		
		const FFrameNumber LowerBound = Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
		const FFrameNumber UpperBound = Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue();
		const int32 Range = UpperBound.Value - LowerBound.Value;
		const int32 CurrentTimeOffset = Sequencer->GetGlobalTime().Time.FrameNumber.Value - LowerBound.Value;
		const float NormalizedTime = static_cast<float>(CurrentTimeOffset) / Range;
		
		const float GameTimeHours = NormalizedTime * DayActor->GetDayLength();
		DayActor->ConditionalSetTimeOfDayPreview(GameTimeHours);
	}
}

#undef LOCTEXT_NAMESPACE
