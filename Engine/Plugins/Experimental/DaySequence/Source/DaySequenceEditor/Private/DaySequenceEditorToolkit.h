// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "IDaySequenceEditorToolkit.h"
#include "Styling/ISlateStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"

class FSpawnTabArgs;

struct FFrameNumber;

class AActor;
class FMenuBuilder;
class ISequencer;
class UActorComponent;
class ADaySequenceActor;
class UDaySequence;
class FDaySequencePlaybackContext;
class FDaySequenceEditorSpawnRegister;
struct FSequencerInitParams;
enum class EMapChangeType : uint8;

/**
 * Implements an Editor toolkit for Day sequences.
 */
class FDaySequenceEditorToolkit
	: public IDaySequenceEditorToolkit
	, public FGCObject
{ 
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FDaySequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/** Virtual destructor */
	virtual ~FDaySequenceEditorToolkit();

public:

	/** Iterate all open Day sequence editor toolkits */
	static void IterateOpenToolkits(TFunctionRef<bool(FDaySequenceEditorToolkit&)> Iter);
	
	/** Iterates over the open toolkits and closes them if Iter returns true. */
	static void CloseOpenToolkits(TFunctionRef<bool(FDaySequenceEditorToolkit&)> Iter);

	/** @return true if any open toolkits are hosted by the sequence editor. */
	static bool HasOpenSequenceEditorToolkits();

	/** @return true if any open toolkits are actor preview. */
	static bool HasOpenActorPreviewToolkits();

	/** Called when a toolkit is opened */
	DECLARE_EVENT_OneParam(FDaySequenceEditorToolkit, FDaySequenceEditorToolkitOpened, FDaySequenceEditorToolkit&);
	static FDaySequenceEditorToolkitOpened& OnOpened();

	/** Called when a toolkit is closed */
	DECLARE_EVENT_OneParam(FDaySequenceEditorToolkit, FDaySequenceEditorToolkitClosed, FDaySequenceEditorToolkit&);
	static FDaySequenceEditorToolkitClosed& OnClosed();

	/** Called when a toolkit is destroyed */
	DECLARE_EVENT_OneParam(FDaySequenceEditorToolkit, FDaySequenceEditorToolkitDestroyed, FDaySequenceEditorToolkit&);
	static FDaySequenceEditorToolkitDestroyed& OnDestroyed();

	/** Called after this class has processed its MapChanged event. */
	DECLARE_EVENT(FDaySequenceEditorToolkit, FDaySequenceEditorToolkitPostMapChanged);
	static FDaySequenceEditorToolkitPostMapChanged& OnToolkitPostMapChanged();

public:

	/**
	 * Initialize this asset editor.
	 *
	 * @param Mode Asset editing mode for this editor (standalone or world-centric).
	 * @param InitToolkitHost When Mode is WorldCentric, this is the level editor instance to spawn this editor within.
	 * @param InDaySequence The animation to edit.
	 */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UDaySequence* InDaySequence);

	/**
	 * Initialize a read-only asset editor for previewing a DaySequenceActor.
	 */
	void InitializeActorPreview(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ADaySequenceActor* InDayActor);

	/** @return true if this toolkit is initialized and open. */
	bool IsActive() const
	{
		// The toolkit is only active if the DaySequence is initialized. It is nulled out during OnClose.
		return DaySequence != nullptr;
	}

	/** @return true if this toolkit was initialized for an Actor Preview. */
	bool IsActorPreview() const
	{
		return PreviewActor != nullptr;
	}

	/**
	 * Get the sequencer object being edited in this tool kit.
	 *
	 * @return Sequencer object.
	 */
	virtual TSharedPtr<ISequencer> GetSequencer() const override
	{
		return Sequencer;
	}

public:

	//~ FAssetEditorToolkit interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(DaySequence);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDaySequenceEditorToolkit");
	}

	virtual void OnClose() override;
	virtual bool CanFindInContentBrowser() const override;
	virtual bool CanSaveAsset() const override { return !IsActorPreview(); }
	virtual bool CanSaveAssetAs() const override { return !IsActorPreview(); }

public:

	//~ IToolkit interface

	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FText GetTabSuffix() const override;

	//~ IAssetEditorInstance interface
	virtual bool IncludeAssetInRestoreOpenAssetsPrompt() const override { return false; }

protected:

	/** Add default movie scene tracks for the given actor. */
	void AddDefaultTracksForActor(AActor& Actor, const FGuid Binding);

	/** Called whenever sequencer has received focus */
	void OnSequencerReceivedFocus();

	/** Called whenever sequencer in initializing tool menu context */
	void OnInitToolMenuContext(FToolMenuContext& MenuContext);

private:
	void InitializeInternal(
		const EToolkitMode::Type Mode,
		const TSharedPtr<IToolkitHost>& InitToolkitHost,
		const FSequencerInitParams& SequencerInitParams,
		TSharedRef<FDaySequenceEditorSpawnRegister>& SpawnRegister);

	void ExtendSequencerToolbar(FName InToolMenuName);

	/** Callback for map changes. */
	void HandleMapChanged(UWorld* NewWorld, EMapChangeType MapChangeType);

	/** Callback for the menu extensibility manager. */
	TSharedRef<FExtender> HandleMenuExtensibilityGetExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects);

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args);

	/** Callback for actor added to sequencer. */
	void HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding);

	/** Callback for keeping Day Sequence Actor preview time correct when editing a day sequence asset. */
	void OnGlobalTimeChanged();
	
private:

	/** Time of day sequence viewed/edited by this toolkit. */
	TObjectPtr<UDaySequence> DaySequence = nullptr;

	/** The sequencer used by this editor. */
	TSharedPtr<ISequencer> Sequencer;

	/** Pointer to the style set to use for toolkits. */
	TSharedRef<ISlateStyle> Style;

	/** Instance of a class used for managing the playback context for a Day sequence. */
	TSharedPtr<FDaySequencePlaybackContext> PlaybackContext;

	/** The actor being previewed if this Toolkit was initialized for an Actor Preview. */
	ADaySequenceActor* PreviewActor = nullptr;

	/** The actor that owns the sequence if this Toolkit was initialized to the actor's root sequence. */
	ADaySequenceActor* RootActor = nullptr;
private:

	/**	The tab ids for all the tabs used */
	static const FName SequencerMainTabId;
};
