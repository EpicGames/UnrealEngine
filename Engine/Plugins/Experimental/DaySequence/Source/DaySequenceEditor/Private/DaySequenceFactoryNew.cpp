// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceFactoryNew.h"
#include "DaySequence.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneTrack.h"
#include "MovieScenePossessable.h"
#include "MovieSceneToolsProjectSettings.h"
#include "DaySequenceActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DaySequenceFactoryNew)

#define LOCTEXT_NAMESPACE "DaySequenceFactory"

UDaySequenceFactoryNew::UDaySequenceFactoryNew()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UDaySequence::StaticClass();
}


UObject* UDaySequenceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UDaySequence* NewDaySequence = NewObject<UDaySequence>(InParent, Name, Flags|RF_Transactional);
	NewDaySequence->Initialize();

	// TODO: Set default playback range from DaySequence project settings
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	const FFrameRate TickResolution = NewDaySequence->GetMovieScene()->GetTickResolution();
	NewDaySequence->GetMovieScene()->SetPlaybackRange((ProjectSettings->DefaultStartTime*TickResolution).FloorToFrame(), (ProjectSettings->DefaultDuration*TickResolution).FloorToFrame().Value);

	AddDefaultBindings(NewDaySequence);
	return NewDaySequence;
}


bool UDaySequenceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

void UDaySequenceFactoryNew::AddDefaultBindings(UDaySequence* NewDaySequence)
{
	static const FString DefaultBindingName = LOCTEXT("DefaultBindingName", "Root Day Sequence Actor").ToString();

	UMovieScene* MovieScene = NewDaySequence->GetMovieScene();

	// Add a default binding
	FGuid PossessableGuid = MovieScene->AddPossessable(DefaultBindingName, ADaySequenceActor::StaticClass());
	NewDaySequence->AddDefaultBinding(PossessableGuid);
}


#undef LOCTEXT_NAMESPACE

