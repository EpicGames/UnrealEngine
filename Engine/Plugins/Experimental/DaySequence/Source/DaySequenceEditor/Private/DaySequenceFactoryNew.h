// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "DaySequenceFactoryNew.generated.h"

class UDaySequence;

/** Implements a factory for UDaySequence objects. */
UCLASS(BlueprintType, hidecategories=Object)
class UDaySequenceFactoryNew : public UFactory
{
	GENERATED_BODY()
public:
	UDaySequenceFactoryNew();
	
	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;

private:

	void AddDefaultBindings(UDaySequence* NewDaySequence);
};
