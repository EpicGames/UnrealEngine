// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequencePlaybackContext.h"
#include "DaySequenceActor.h"
#include "DaySequenceEditorSettings.h"
#include "DaySequence.h"
#include "Engine/Level.h"
#include "IDaySequenceEditorModule.h"

#include "Engine/NetDriver.h"
#include "Engine/World.h"

#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieSceneCaptureDialogModule.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "DaySequencePlaybackContext"

class SDaySequenceContextPicker : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSetPlaybackContext, ADaySequenceActor*);

	SLATE_BEGIN_ARGS(SDaySequenceContextPicker){}

		/** Attribute for retrieving the bound Day sequence */
		SLATE_ATTRIBUTE(UDaySequence*, Owner)

		/** Attribute for retrieving the current context */
		SLATE_ATTRIBUTE(ADaySequenceActor*, OnGetPlaybackContext)

		/** Called when the user explicitly chooses a new context */
		SLATE_EVENT(FOnSetPlaybackContext, OnSetPlaybackContext)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	TSharedRef<SWidget> BuildWorldPickerMenu();

	static FText GetContextDescription(const ADaySequenceActor* Context);
	static FText GetWorldDescription(const UWorld* World);

	FText GetCurrentContextText() const
	{
		const ADaySequenceActor* CurrentContext = PlaybackContextAttribute.Get();
		return GetContextDescription(CurrentContext);
	}

	const FSlateBrush* GetBorderBrush() const
	{
		const ADaySequenceActor* CurrentContext = PlaybackContextAttribute.Get();
		if (CurrentContext)
		{
			UWorld* CurrentWorld = CurrentContext->GetWorld();
			check(CurrentWorld);

			if (CurrentWorld->WorldType == EWorldType::PIE)
			{
				return GEditor->bIsSimulatingInEditor ? FAppStyle::GetBrush("LevelViewport.StartingSimulateBorder") : FAppStyle::GetBrush("LevelViewport.StartingPlayInEditorBorder");
			}
		}

		return FStyleDefaults::GetNoBrush();
	}

	void ToggleAutoPIE() const
	{
		UDaySequenceEditorSettings* Settings = GetMutableDefault<UDaySequenceEditorSettings>();
		Settings->bAutoBindToPIE = !Settings->bAutoBindToPIE;
		Settings->SaveConfig();

		OnSetPlaybackContextEvent.ExecuteIfBound(nullptr);
	}

	bool IsAutoPIEChecked() const
	{
		return GetDefault<UDaySequenceEditorSettings>()->bAutoBindToPIE;
	}

	void ToggleAutoSimulate() const
	{
		UDaySequenceEditorSettings* Settings = GetMutableDefault<UDaySequenceEditorSettings>();
		Settings->bAutoBindToSimulate = !Settings->bAutoBindToSimulate;
		Settings->SaveConfig();

		OnSetPlaybackContextEvent.ExecuteIfBound(nullptr);
	}

	bool IsAutoSimulateChecked() const
	{
		return GetDefault<UDaySequenceEditorSettings>()->bAutoBindToSimulate;
	}

	void OnSetPlaybackContext(TWeakObjectPtr<ADaySequenceActor> InContext)
	{
		if (ADaySequenceActor* NewContext = InContext.Get())
		{
			OnSetPlaybackContextEvent.ExecuteIfBound(NewContext);
		}
	}

	bool IsCurrentPlaybackContext(TWeakObjectPtr<ADaySequenceActor> InContext)
	{
		ADaySequenceActor* Context = PlaybackContextAttribute.Get();
		return (InContext == Context);
	}

private:
	TAttribute<UDaySequence*> OwnerAttribute;
	TAttribute<ADaySequenceActor*> PlaybackContextAttribute;
	FOnSetPlaybackContext OnSetPlaybackContextEvent;
};

namespace UE
{
namespace MovieScene
{

/**
 * Finds all Day sequence actors in the given world, and return those that point to the given sequence, or any day sequence actor as a fallback
 */
static void FindDaySequenceActors(const UWorld* InWorld, const UDaySequence* InDaySequence, TArray<ADaySequenceActor*>& OutActors)
{
	ADaySequenceActor* Fallback = nullptr;

	for (const ULevel* Level : InWorld->GetLevels())
	{
		for (AActor* Actor : Level->Actors)
		{
			ADaySequenceActor* DayActor = Cast<ADaySequenceActor>(Actor);
			if (!DayActor)
			{
				continue;
			}

			Fallback = DayActor;
			if (DayActor->GetRootSequence() == InDaySequence || DayActor->ContainsDaySequence(InDaySequence))
			{
				OutActors.Add(DayActor);
			}
		}
	}

	if (OutActors.Num() == 0 && Fallback != nullptr)
	{
		OutActors.Add(Fallback);
	}
}

}
}

FDaySequencePlaybackContext::FDaySequencePlaybackContext(UDaySequence* InDaySequence)
	: DaySequence(InDaySequence)
{
	FEditorDelegates::MapChange.AddRaw(this, &FDaySequencePlaybackContext::OnMapChange);
	FEditorDelegates::PreBeginPIE.AddRaw(this, &FDaySequencePlaybackContext::OnPieEvent);
	FEditorDelegates::BeginPIE.AddRaw(this, &FDaySequencePlaybackContext::OnPieEvent);
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FDaySequencePlaybackContext::OnPieEvent);
	FEditorDelegates::PrePIEEnded.AddRaw(this, &FDaySequencePlaybackContext::OnPieEvent);
	FEditorDelegates::EndPIE.AddRaw(this, &FDaySequencePlaybackContext::OnPieEvent);

	if (GEngine)
	{
		GEngine->OnWorldAdded().AddRaw(this, &FDaySequencePlaybackContext::OnWorldListChanged);
		GEngine->OnWorldDestroyed().AddRaw(this, &FDaySequencePlaybackContext::OnWorldListChanged);
	}
}

FDaySequencePlaybackContext::~FDaySequencePlaybackContext()
{
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PreBeginPIE.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnWorldAdded().RemoveAll(this);
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}
}

void FDaySequencePlaybackContext::OnPieEvent(bool)
{
	WeakCurrentContext = nullptr;
}

void FDaySequencePlaybackContext::OnMapChange(uint32)
{
	WeakCurrentContext = nullptr;
}

void FDaySequencePlaybackContext::OnWorldListChanged(UWorld*)
{
	WeakCurrentContext = nullptr;
}

UDaySequence* FDaySequencePlaybackContext::GetDaySequence() const
{
	return DaySequence.Get();
}

ADaySequenceActor* FDaySequencePlaybackContext::GetPlaybackContext() const
{
	UpdateCachedContext();
	return WeakCurrentContext.Get();
}

UObject* FDaySequencePlaybackContext::GetPlaybackContextAsObject() const
{
	return GetPlaybackContext();
}

ADaySequenceActor* FDaySequencePlaybackContext::GetPlaybackClient() const
{
	UpdateCachedContext();
	return WeakCurrentContext.Get();
}

IMovieScenePlaybackClient* FDaySequencePlaybackContext::GetPlaybackClientAsInterface() const
{
	return GetPlaybackClient();
}

void FDaySequencePlaybackContext::OverrideWith(ADaySequenceActor* InNewContext)
{
	// InNewContext may be null to force an auto update
	WeakCurrentContext = InNewContext;
}

TSharedRef<SWidget> FDaySequencePlaybackContext::BuildWorldPickerCombo()
{
	return SNew(SDaySequenceContextPicker)
		.Owner(this, &FDaySequencePlaybackContext::GetDaySequence)
		.OnGetPlaybackContext(this, &FDaySequencePlaybackContext::GetPlaybackContext)
		.OnSetPlaybackContext(this, &FDaySequencePlaybackContext::OverrideWith);
}

ADaySequenceActor* FDaySequencePlaybackContext::ComputePlaybackContext(const UDaySequence* InDaySequence)
{
	const UDaySequenceEditorSettings* Settings            = GetDefault<UDaySequenceEditorSettings>();
	IMovieSceneCaptureDialogModule*   CaptureDialogModule = FModuleManager::GetModulePtr<IMovieSceneCaptureDialogModule>("MovieSceneCaptureDialog");

	// Some plugins may not want us to automatically attempt to bind to the world where it doesn't make sense,
	// such as movie rendering.
	bool bAllowPlaybackContextBinding = true;
	IDaySequenceEditorModule* DaySequenceEditorModule = FModuleManager::GetModulePtr<IDaySequenceEditorModule>("DaySequenceEditor");
	if (DaySequenceEditorModule)
	{
		DaySequenceEditorModule->OnComputePlaybackContext().Broadcast(bAllowPlaybackContextBinding);
	}

	UWorld* RecordingWorld = CaptureDialogModule ? CaptureDialogModule->GetCurrentlyRecordingWorld() : nullptr;

	// Only allow PIE and Simulate worlds if the settings allow them
	const bool bIsSimulatingInEditor = GEditor && GEditor->bIsSimulatingInEditor;
	const bool bIsPIEValid           = (!bIsSimulatingInEditor && Settings->bAutoBindToPIE) || ( bIsSimulatingInEditor && Settings->bAutoBindToSimulate);

	UWorld* EditorWorld = nullptr;

	// Return PIE worlds if there are any
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			UWorld* ThisWorld = Context.World();
			const bool bIsServerWorld = (ThisWorld && ThisWorld->GetNetDriver() && ThisWorld->GetNetDriver()->IsServer());
			if (bIsPIEValid && bAllowPlaybackContextBinding && RecordingWorld != ThisWorld && !bIsServerWorld)
			{
				TArray<ADaySequenceActor*> DaySequenceActors;
				UE::MovieScene::FindDaySequenceActors(ThisWorld, InDaySequence, DaySequenceActors);

				if (DaySequenceActors.Num() > 0)
				{
					return DaySequenceActors[0];
				}
			}
		}
		else if (Context.WorldType == EWorldType::Editor)
		{
			EditorWorld = Context.World();
		}
	}

	if (ensure(EditorWorld))
	{
		TArray<ADaySequenceActor*> DaySequenceActors;
		UE::MovieScene::FindDaySequenceActors(EditorWorld, InDaySequence, DaySequenceActors);

		if (DaySequenceActors.Num() > 0)
		{
			return DaySequenceActors[0];
		}
	}

	return nullptr;
}

void FDaySequencePlaybackContext::UpdateCachedContext() const
{
	if (WeakCurrentContext.Get() != nullptr)
	{
		return;
	}

	WeakCurrentContext = ComputePlaybackContext(DaySequence.Get());
}

void SDaySequenceContextPicker::Construct(const FArguments& InArgs)
{
	OwnerAttribute = InArgs._Owner;
	PlaybackContextAttribute = InArgs._OnGetPlaybackContext;
	OnSetPlaybackContextEvent = InArgs._OnSetPlaybackContext;
	
	check(OwnerAttribute.IsSet());
	check(PlaybackContextAttribute.IsSet());
	check(OnSetPlaybackContextEvent.IsBound());

	ChildSlot
	.Padding(0.0f)
	[
		SNew(SBorder)
		.BorderImage(this, &SDaySequenceContextPicker::GetBorderBrush)
		.Padding(FMargin(4.f, 0.f))
		[
			SNew(SComboButton)
			.ContentPadding(0)
			.ForegroundColor(FSlateColor::UseForeground())
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.OnGetMenuContent(this, &SDaySequenceContextPicker::BuildWorldPickerMenu)
			.ToolTipText(FText::Format(LOCTEXT("WorldPickerTextFomrat", "'{0}': The actor to use for previewing the effects of this day sequence."), GetCurrentContextText()))
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.World"))
			]
		]
	];
}

FText SDaySequenceContextPicker::GetContextDescription(const ADaySequenceActor* Context)
{
	if (!Context)
	{
		return LOCTEXT("InvalidPlaybackContext", "<< invalid >>");
	}

	UWorld* World = Context->GetWorld();
	const FText WorldDescription = GetWorldDescription(World);

	return FText::Format(
			LOCTEXT("PlaybackContextDescription", "{0} ({1})"), 
			FText::FromString(Context->GetName()),
			WorldDescription
			);
}

FText SDaySequenceContextPicker::GetWorldDescription(const UWorld* World)
{
	FText PostFix;
	if (World->WorldType == EWorldType::PIE)
	{
		switch(World->GetNetMode())
		{
		case NM_Client:
			PostFix = FText::Format(LOCTEXT("ClientPostfixFormat", " (Client {0})"), FText::AsNumber(World->GetOutermost()->GetPIEInstanceID() - 1));
			break;
		case NM_DedicatedServer:
		case NM_ListenServer:
			PostFix = LOCTEXT("ServerPostfix", " (Server)");
			break;
		case NM_Standalone:
			PostFix = GEditor->bIsSimulatingInEditor ? LOCTEXT("SimulateInEditorPostfix", " (Simulate)") : LOCTEXT("PlayInEditorPostfix", " (PIE)");
			break;
		default:
			break;
		}
	}
	else if (World->WorldType == EWorldType::Editor)
	{
		PostFix = LOCTEXT("EditorPostfix", " (Editor)");
	}

	return FText::Format(LOCTEXT("WorldFormat", "{0}{1}"), FText::FromString(World->GetFName().GetPlainNameString()), PostFix);
}

TSharedRef<SWidget> SDaySequenceContextPicker::BuildWorldPickerMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const UDaySequence* DaySequence = OwnerAttribute.Get();

	const UDaySequenceEditorSettings* Settings = GetDefault<UDaySequenceEditorSettings>();
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ActorsHeader", "Actors"));
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World == nullptr || (Context.WorldType != EWorldType::PIE && Context.WorldType != EWorldType::Editor))
			{
				continue;
			}

			bool bFoundActors = false;
			if (DaySequence)
			{
				TArray<ADaySequenceActor*> DaySequenceActors;
				UE::MovieScene::FindDaySequenceActors(World, DaySequence, DaySequenceActors);
				bFoundActors = DaySequenceActors.Num() > 0;

				for (ADaySequenceActor* DaySequenceActor : DaySequenceActors)
				{
					MenuBuilder.AddMenuEntry(
							GetContextDescription(DaySequenceActor),
							FText(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(
									this, 
									&SDaySequenceContextPicker::OnSetPlaybackContext, 
									MakeWeakObjectPtr(DaySequenceActor)),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(
									this, 
									&SDaySequenceContextPicker::IsCurrentPlaybackContext, 
									MakeWeakObjectPtr(DaySequenceActor))
								),
							NAME_None,
							EUserInterfaceActionType::RadioButton
							);
				}
			}
			
			if (!bFoundActors)
			{
				MenuBuilder.AddMenuEntry(
						GetContextDescription(nullptr),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(
								this, 
								&SDaySequenceContextPicker::OnSetPlaybackContext, 
								MakeWeakObjectPtr<ADaySequenceActor>(nullptr)),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(
								this, 
								&SDaySequenceContextPicker::IsCurrentPlaybackContext, 
								MakeWeakObjectPtr<ADaySequenceActor>(nullptr))
							),
						NAME_None,
						EUserInterfaceActionType::RadioButton
						);
			}
		}
	}
	MenuBuilder.EndSection();


	MenuBuilder.BeginSection(NAME_None, LOCTEXT("OptionsHeader", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoBindPIE_Label", "Auto Bind to PIE"),
			LOCTEXT("AutoBindPIE_Tip", "Automatically binds an active Sequencer window to the current PIE world, if available."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDaySequenceContextPicker::ToggleAutoPIE),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDaySequenceContextPicker::IsAutoPIEChecked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoBindSimulate_Label", "Auto Bind to Simulate"),
			LOCTEXT("AutoBindSimulate_Tip", "Automatically binds an active Sequencer window to the current Simulate world, if available."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDaySequenceContextPicker::ToggleAutoSimulate),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDaySequenceContextPicker::IsAutoSimulateChecked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
