// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

class SWidget;

class ADaySequenceActor;
class IMovieScenePlaybackClient;
class UDaySequence;
class UWorld;

/**
 * Class that manages the current playback context that a Time of Day editor should use for playback
 */
class FDaySequencePlaybackContext : public TSharedFromThis<FDaySequencePlaybackContext>
{
public:

	FDaySequencePlaybackContext(UDaySequence* InDaySequence);
	~FDaySequencePlaybackContext();

	/**
	 * Gets the Day sequence for which we are trying to find the context.
	 */
	UDaySequence* GetDaySequence() const;

	/**
	 * Build a world picker widget that allows the user to choose a world, and exit the auto-bind settings
	 */
	TSharedRef<SWidget> BuildWorldPickerCombo();

	/**
	 * Resolve the current world context pointer. Can never be nullptr.
	 */
	ADaySequenceActor* GetPlaybackContext() const;

	/**
	 * Returns GetPlaybackContext as a plain object.
	 */
	UObject* GetPlaybackContextAsObject() const;

	/**
	 * Resolve the current playback client. May be nullptr.
	 */
	ADaySequenceActor* GetPlaybackClient() const;

	/**
	 * Returns GetPlaybackClient as an interface pointer.
	 */
	IMovieScenePlaybackClient* GetPlaybackClientAsInterface() const;

	/**
	 * Specify a new world to use as the context. Persists until the next PIE or map change event.
	 * May be null, in which case the context will be recomputed automatically
	 */
	void OverrideWith(ADaySequenceActor* InNewClient);

private:

	/**
	 * Compute the new playback context based on the user's current auto-bind settings.
	 * Will use the first encountered PIE or Simulate world if possible, else the Editor world as a fallback
	 */
	static ADaySequenceActor* ComputePlaybackContext(const UDaySequence* InDaySequence);

	/**
	 * Update the cached context and client pointers if needed.
	 */
	void UpdateCachedContext() const;

	void OnPieEvent(bool);
	void OnMapChange(uint32);
	void OnWorldListChanged(UWorld*);

private:

	/** Time of Day sequence that we should find a context for */
	TWeakObjectPtr<UDaySequence> DaySequence;

	/** Mutable cached context pointer */
	mutable TWeakObjectPtr<ADaySequenceActor> WeakCurrentContext;
};
