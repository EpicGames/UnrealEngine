// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceTimeDetailsCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "DaySequenceTime.h"
#include "PropertyHandle.h"
#include "Trace/Detail/Channel.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "Time"

void FDaySequenceTimeDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TimeProperty = PropertyHandle;

	ChildBuilder.AddProperty(PropertyHandle)
		.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FDaySequenceTimeDetailsCustomization::OnGetTimeText)
			.OnTextCommitted(this, &FDaySequenceTimeDetailsCustomization::OnTimeTextCommitted)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsReadOnly_Lambda([this]()
			{
				return TimeProperty.IsValid() ? !TimeProperty->IsEditable() : false;
			})
		];
}

FText FDaySequenceTimeDetailsCustomization::OnGetTimeText() const
{
	TArray<void*> RawData;
	TimeProperty->AccessRawData(RawData);

	if (RawData.Num())
	{
		FString CurrentValue = ((FDaySequenceTime*)RawData[0])->ToString();
		return FText::FromString(CurrentValue);
	}

	return FText::GetEmpty();
}

void FDaySequenceTimeDetailsCustomization::OnTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	TArray<void*> RawData;
	TimeProperty->AccessRawData(RawData);

	if (RawData.Num())
	{
		TArray<FString> Splits;
		InText.ToString().ParseIntoArray(Splits, TEXT(":"));

		const int NumSplits = Splits.Num();
		if (NumSplits > 0 && NumSplits <= 3)
		{
			GEditor->BeginTransaction(FText::Format(LOCTEXT("SetTimeProperty", "Edit {0}"), TimeProperty->GetPropertyDisplayName()));
			
			TimeProperty->NotifyPreChange();
			
			((FDaySequenceTime*)RawData[0])->Hours   = FCString::Atoi(*Splits[0]);
			((FDaySequenceTime*)RawData[0])->Minutes = NumSplits > 1 ? FCString::Atoi(*Splits[1]) : 0;
			((FDaySequenceTime*)RawData[0])->Seconds = NumSplits > 2 ? FCString::Atoi(*Splits[2]) : 0;
			
			TimeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			TimeProperty->NotifyFinishedChangingProperties();
			
			GEditor->EndTransaction();
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Unexpected time format. Expected 1-3 values, got %d", Splits.Num());
		}
	}
}

#undef LOCTEXT_NAMESPACE
