// Copyright Epic Games, Inc. All Rights Reserved.

#include "DaySequenceTrackEditor.h"
#include "DaySequenceTrack.h"

#define LOCTEXT_NAMESPACE "FDaySequenceTrackEditor"

FDaySequenceTrackEditor::FDaySequenceTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FSubTrackEditor(InSequencer)
{
}

FDaySequenceTrackEditor::~FDaySequenceTrackEditor()
{
}

TSharedRef<ISequencerTrackEditor> FDaySequenceTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FDaySequenceTrackEditor(InSequencer));
}

// ISequencerTrackEditor interface
bool FDaySequenceTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	const ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UDaySequenceTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

// FSubTrackEditor interface
TSubclassOf<UMovieSceneSubTrack> FDaySequenceTrackEditor::GetSubTrackClass() const
{
	return UDaySequenceTrack::StaticClass();
}

void FDaySequenceTrackEditor::GetSupportedSequenceClassPaths(TArray<FTopLevelAssetPath>& ClassPaths) const
{
	ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/DaySequence"), TEXT("DaySequence")));
}


#undef LOCTEXT_NAMESPACE
