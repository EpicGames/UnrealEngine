// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackEditors/SubTrackEditor.h"

class FDaySequenceTrackEditor	: public FSubTrackEditor
{
public:
	FDaySequenceTrackEditor(TSharedRef<ISequencer> InSequencer);
	virtual ~FDaySequenceTrackEditor() override;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:
	// ISequencerTrackEditor interface
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	// FSubTrackEditor interface
	virtual TSubclassOf<UMovieSceneSubTrack> GetSubTrackClass() const;
	virtual void GetSupportedSequenceClassPaths(TArray<FTopLevelAssetPath>& ClassPaths) const override;
};


