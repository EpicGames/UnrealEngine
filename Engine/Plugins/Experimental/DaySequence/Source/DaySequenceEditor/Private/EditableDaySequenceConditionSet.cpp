// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditableDaySequenceConditionSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableDaySequenceConditionSet)

UEditableDaySequenceConditionSet::UEditableDaySequenceConditionSet()
: ConditionsProperty(nullptr)
{
	if (!IsTemplate())
	{
		ConditionsProperty = FindFProperty<FProperty>(GetClass(), TEXT("ConditionSet"));
	}
}

void UEditableDaySequenceConditionSet::SetConditions(const FDaySequenceConditionSet::FConditionValueMap& InConditions)
{
	ConditionSet.Conditions = InConditions;
}

FDaySequenceConditionSet::FConditionValueMap& UEditableDaySequenceConditionSet::GetConditions()
{
	return ConditionSet.Conditions;
}
	
FString UEditableDaySequenceConditionSet::GetConditionSetExportText()
{
	ConditionsPropertyAsString.Reset();
		
	if (ConditionsProperty)
	{
		ConditionsProperty->ExportTextItem_Direct(ConditionsPropertyAsString, &ConditionSet, &ConditionSet, this, 0);
	}

	return ConditionsPropertyAsString;
}
