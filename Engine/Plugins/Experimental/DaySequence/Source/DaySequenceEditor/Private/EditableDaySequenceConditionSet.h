// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DaySequenceConditionSet.h"

#include "EditableDaySequenceConditionSet.generated.h"

UCLASS(Transient, MinimalAPI) 
class UEditableDaySequenceConditionSet : public UObject
{
public:
	GENERATED_BODY()

	UEditableDaySequenceConditionSet();
	
	void SetConditions(const FDaySequenceConditionSet::FConditionValueMap& InConditions);
	FDaySequenceConditionSet::FConditionValueMap& GetConditions();
	
	FString GetConditionSetExportText();

private:
	UPROPERTY()
	FDaySequenceConditionSet ConditionSet;
	
	FProperty* ConditionsProperty;
	FString ConditionsPropertyAsString;
};