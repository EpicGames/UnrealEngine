// Copyright Epic Games, Inc. All Rights Reserved.
#include "EnvironmentLightingActorDetails.h"

#include "DetailLayoutBuilder.h"

TSharedRef<IDetailCustomization> FEnvironmentLightingActorDetails::MakeInstance()
{
	return MakeShared<FEnvironmentLightingActorDetails>();
}

void FEnvironmentLightingActorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Hide the Environment category that holds our subobject components since that UI is not useful.
	DetailLayout.HideCategory("Environment");
	
	DetailLayout.HideCategory("Rendering");
	DetailLayout.HideCategory("Physics");
	DetailLayout.HideCategory("HLOD");
	DetailLayout.HideCategory("Activation");
	DetailLayout.HideCategory("Input");
	DetailLayout.HideCategory("Collision");
	DetailLayout.HideCategory("Actor");
	DetailLayout.HideCategory("Lod");
	DetailLayout.HideCategory("Cooking");
	DetailLayout.HideCategory("DataLayers");
	DetailLayout.HideCategory("WorldPartition");
}