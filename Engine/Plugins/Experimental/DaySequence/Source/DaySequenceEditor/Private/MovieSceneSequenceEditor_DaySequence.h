// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "MovieSceneSequenceEditor.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DaySequenceDirector.h"
#include "DaySequence.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"

struct FMovieSceneSequenceEditor_DaySequence : FMovieSceneSequenceEditor
{
	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const
	{
		return true;
	}

	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const override
	{
#if WITH_EDITOR
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
		const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
		FClassViewerInitializationOptions ClassViewerOptions = {};

		if (GlobalClassFilter.IsValid())
		{
			return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, UDaySequenceDirector::StaticClass(), ClassFilterFuncs);
		}
#endif
		return true;
	}

	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		UDaySequence* DaySequence = CastChecked<UDaySequence>(InSequence);
		return DaySequence->GetDirectorBlueprint();
	}

	virtual UBlueprint* CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		UBlueprint* Blueprint = GetBlueprintForSequence(InSequence);
		if (!ensureMsgf(!Blueprint, TEXT("Should not call CreateBlueprintForSequence when one already exists")))
		{
			return Blueprint;
		}

		UDaySequence* DaySequence = CastChecked<UDaySequence>(InSequence);

		Blueprint = FKismetEditorUtilities::CreateBlueprint(UDaySequenceDirector::StaticClass(), InSequence, FName(*DaySequence->GetDirectorBlueprintName()), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());
		Blueprint->ClearFlags(RF_Standalone);

		DaySequence->SetDirectorBlueprint(Blueprint);
		return Blueprint;
	}
};
