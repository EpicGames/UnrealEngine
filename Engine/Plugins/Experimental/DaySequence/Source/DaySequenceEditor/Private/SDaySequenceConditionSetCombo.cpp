// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDaySequenceConditionSetCombo.h"

#include "DaySequenceConditionSet.h"
#include "DaySequenceConditionTag.h"
#include "EditableDaySequenceConditionSet.h"
#include "PropertyHandle.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SDaySequenceConditionSetPicker.h"
#include "SDaySequenceConditionTagChip.h"
#include "UObject/Package.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "DaySequenceConditionSetCombo"

//------------------------------------------------------------------------------
// SDaySequenceConditionSetCombo
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SDaySequenceConditionSetCombo)
void SDaySequenceConditionSetCombo::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{

}

void SDaySequenceConditionSetCombo::Construct(const FArguments& InArgs)
{
	StructPropertyHandle = InArgs._StructPropertyHandle;

	HelperConditionSet.Reset(NewObject<UEditableDaySequenceConditionSet>(GetTransientPackage(), NAME_None, RF_Transient));
	
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle())
	{
		StructPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SDaySequenceConditionSetCombo::RefreshListView));
		RefreshListView();

		TWeakPtr<SDaySequenceConditionSetCombo> WeakSelf = StaticCastWeakPtr<SDaySequenceConditionSetCombo>(AsWeak());

		ActiveConditionTagsListView = SNew(SListView<UClass*>)
			.ListItemsSource(&ActiveConditionTags)
			.SelectionMode(ESelectionMode::None)
			.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
			.OnGenerateRow(this, &SDaySequenceConditionSetCombo::OnGenerateRow)
			.Visibility_Lambda([WeakSelf]()
			{
				if (const TSharedPtr<SDaySequenceConditionSetCombo> Self = WeakSelf.Pin())
				{
					return Self->ActiveConditionTags.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				}
				return EVisibility::Collapsed;
			});

		ChildSlot
		[
			SNew(SHorizontalBox)
				
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			[
				SAssignNew(ComboButton, SComboButton)
				.HasDownArrow(true)
				.VAlign(VAlign_Top)
				.ContentPadding(0)
				.OnGetMenuContent(this, &SDaySequenceConditionSetCombo::OnGetMenuContent)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)

						// Condition Tag List
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Top)
						.AutoWidth()
						[
							ActiveConditionTagsListView.ToSharedRef()
						]
						
						// Empty indicator
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(4, 2))
						[
							SNew(SBox)
							.HeightOverride(SDaySequenceConditionTagChip::ChipHeight)
							.VAlign(VAlign_Center)
							.Padding(0, 0, 8, 0)
							.Visibility_Lambda([WeakSelf]()
							{
								if (const TSharedPtr<SDaySequenceConditionSetCombo> Self = WeakSelf.Pin())
								{
									return Self->ActiveConditionTags.Num() > 0 ? EVisibility::Collapsed : EVisibility::Visible;
								}
								return EVisibility::Collapsed;
							})
							[
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
								.Text(LOCTEXT("DaySequenceConditionSetCombo_Empty", "Empty"))
								.ToolTipText(LOCTEXT("DaySequenceConditionSetCombo_EmptyTooltip", "Empty Condition Set"))
							]
						]
					]
				]
			]
		];
	}
}

TSharedRef<ITableRow> SDaySequenceConditionSetCombo::OnGenerateRow(UClass* InCondition, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<UClass*>, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row"))
		.Padding(FMargin(0,2))
		[
			SNew(SDaySequenceConditionTagChip)
			.TagClass(InCondition)
			.Text(FText::FromString(Cast<UDaySequenceConditionTag>(InCondition->GetDefaultObject())->GetConditionName()))
			.ToolTipText(FText::FromString(InCondition->GetClassPathName().ToString()))
			.OnClearPressed(this, &SDaySequenceConditionSetCombo::OnClearTagClicked, InCondition)
			.OnExpectedValueChanged(this, &SDaySequenceConditionSetCombo::OnConditionExpectedValueChanged)
			.ExpectedValue_Lambda([this, InCondition]()
			{
				return GetConditionExpectedValue(InCondition);
			})
		];
}

TSharedRef<SWidget> SDaySequenceConditionSetCombo::OnGetMenuContent()
{
	TagPicker = SNew(SDaySequenceConditionSetPicker)
	.StructPropertyHandle(StructPropertyHandle);

	ComboButton->SetMenuContentWidgetToFocus(TagPicker);

	return TagPicker.ToSharedRef();
}

FReply SDaySequenceConditionSetCombo::OnClearTagClicked(UClass* InCondition)
{
	HelperConditionSet->GetConditions().Remove(TSubclassOf<UDaySequenceConditionTag>(InCondition));

	// Set the property with a formatted string in order to propagate CDO changes to instances if necessary
	const FString OutString = HelperConditionSet->GetConditionSetExportText();
	StructPropertyHandle->SetValueFromFormattedString(OutString);
		
	RefreshListView();
		
	return FReply::Handled();
}

void SDaySequenceConditionSetCombo::OnConditionExpectedValueChanged(UClass* InCondition, bool bNewPassValue)
{
	StructPropertyHandle->NotifyPreChange();
	
	void* StructPointer = nullptr;
	if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
	{
		FDaySequenceConditionSet& ConditionSet = *static_cast<FDaySequenceConditionSet*>(StructPointer);
		FDaySequenceConditionSet::FConditionValueMap& Conditions = ConditionSet.Conditions;

		const TSubclassOf<UDaySequenceConditionTag> Subclass(InCondition);
		bool* ExpectedValue = Conditions.Find(Subclass);
    
		// This is a nullptr/IsChildOf check on InTag and a nullptr check on ExpectedValue
		if (*Subclass && ExpectedValue)
		{
			*ExpectedValue = bNewPassValue;
			StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}

	StructPropertyHandle->NotifyFinishedChangingProperties();
}

bool SDaySequenceConditionSetCombo::GetConditionExpectedValue(UClass* InCondition)
{
	void* StructPointer = nullptr;
	if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
	{
		FDaySequenceConditionSet& ConditionSet = *static_cast<FDaySequenceConditionSet*>(StructPointer);
		FDaySequenceConditionSet::FConditionValueMap& Conditions = ConditionSet.Conditions;

		const TSubclassOf<UDaySequenceConditionTag> Subclass(InCondition);
		const bool* ExpectedValue = Conditions.Find(Subclass);

		// This is a nullptr/IsChildOf check on InTag and a nullptr check on ExpectedValue
		if (*Subclass && ExpectedValue)
		{
			return *ExpectedValue;
		}
	}
	
	return true;
}

void SDaySequenceConditionSetCombo::RefreshListView()
{
	ActiveConditionTags.Reset();
	
	// Add UClass* to our TagsToEdit list from the property handle
	void* StructPointer = nullptr;
	if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
	{
		FDaySequenceConditionSet& ConditionSet = *static_cast<FDaySequenceConditionSet*>(StructPointer);
		FDaySequenceConditionSet::FConditionValueMap& Conditions = ConditionSet.Conditions;

		HelperConditionSet->SetConditions(Conditions);
		
		for (TTuple<TSubclassOf<UDaySequenceConditionTag>, bool>& Element : Conditions)
		{
			TSubclassOf<UDaySequenceConditionTag>& Subclass = Element.Key;
			ActiveConditionTags.AddUnique(*Subclass);
		}
	}

	// Lexicographically sort condition tags.
	Algo::Sort(ActiveConditionTags, [](const UClass* LHS, const UClass* RHS)
	{
		const TObjectPtr<UDaySequenceConditionTag> LHSCDO = Cast<UDaySequenceConditionTag>(LHS->GetDefaultObject());
		const TObjectPtr<UDaySequenceConditionTag> RHSCDO = Cast<UDaySequenceConditionTag>(RHS->GetDefaultObject());
			
		check (IsValid(LHSCDO) && IsValid(RHSCDO));
			
		return LHSCDO->GetConditionName() < RHSCDO->GetConditionName();
	});
	
	// Refresh the slate list
	if (ActiveConditionTagsListView.IsValid())
	{
		ActiveConditionTagsListView->SetItemsSource(&ActiveConditionTags);
		ActiveConditionTagsListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
