// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class UEditableDaySequenceConditionSet;
template <typename ItemType> class SListView;

class IPropertyHandle;
class SMenuAnchor;
class ITableRow;
class STableViewBase;
class SDaySequenceConditionSetPicker;
class SComboButton;

/**
 * Widget for editing a condition set.
 */
class SDaySequenceConditionSetCombo : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDaySequenceConditionSetCombo, SCompoundWidget)
	
public:
	
	SLATE_BEGIN_ARGS(SDaySequenceConditionSetCombo)
		: _StructPropertyHandle(nullptr)
	{}
		// Used for writing changes to the condition set being edited. 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, StructPropertyHandle)
	SLATE_END_ARGS();
	
	DAYSEQUENCEEDITOR_API void Construct(const FArguments& InArgs);

private:
	/** Returns a table row with an SDaySequenceConditionTagChip. */
	TSharedRef<ITableRow> OnGenerateRow(UClass* InCondition, const TSharedRef<STableViewBase>& OwnerTable);

	/** Instantiates the tag picker and sets it as the widget to focus for the ComboButton. */
	TSharedRef<SWidget> OnGetMenuContent();

	/** Bound to/called via SDaySequenceConditionTagChip::OnClearPressed. Removes TagToClear from condition set. */
	FReply OnClearTagClicked(UClass* InCondition);

	/** Bound to/called via SDaySequenceConditionTagChip::OnExpectedValueChanged. Sets the expected value for TagToModify to bNewPassValue. */
	void OnConditionExpectedValueChanged(UClass* InCondition, bool bNewPassValue);

	bool GetConditionExpectedValue(UClass* InCondition);
	
	/** Populates ActiveConditionTags with the conditions currently active on the condition set. */
	void RefreshListView();

	/** The set of condition tags to display, based on the condition tags present in the condition set we are editing. */
	TArray<UClass*> ActiveConditionTags;
	TSharedPtr<SListView<UClass*>> ActiveConditionTagsListView;

	/** Widgets we retain ownership of and refer to in a named manner. */
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SDaySequenceConditionSetPicker> TagPicker;

	/** Property handle to an FDaySequenceConditionSet, used for accessing the source condition set. */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** A helper class which is used for propagating changes to the source condition set. */
	TStrongObjectPtr<UEditableDaySequenceConditionSet> HelperConditionSet;
};
