// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDaySequenceConditionSetPicker.h"

#include "Algo/ForEach.h"
#include "AssetRegistry/ARFilter.h"
#include "DaySequenceConditionTag.h"
#include "EditableDaySequenceConditionSet.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "PropertyHandle.h"
#include "SDaySequenceConditionTagChip.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"

// Asset registry querying
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"	// For FBlueprintTags::NativeParentClassPath
#include "AssetRegistry/AssetData.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DaySequenceConditionSetPicker"

//------------------------------------------------------------------------------
// SDaySequenceConditionSetPicker
//------------------------------------------------------------------------------

void SDaySequenceConditionSetPicker::Construct(const FArguments& InArgs)
{
	StructPropertyHandle = InArgs._StructPropertyHandle;
	
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle())
	{
		PopulateVisibleClasses();

		HelperConditionSet.Reset(NewObject<UEditableDaySequenceConditionSet>(GetTransientPackage(), NAME_None, RF_Transient));
		
		StructPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SDaySequenceConditionSetPicker::PopulateCheckedTags));
		PopulateCheckedTags();

		OnSearchStringChanged(FText());
		
		ChildSlot
		[
			GetChildWidget()
		];
	}
}

TSharedRef<SWidget> SDaySequenceConditionSetPicker::GetChildWidget()
{
	const TSharedRef<SWidget> MenuContent = SNew(SBox)
	[
		SNew(SVerticalBox)

		// Search box
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		[
			SAssignNew(ConditionSearchBox, SSearchBox)
			.HintText(LOCTEXT("DaySequenceConditionSetPicker_SearchBoxHint", "Search Condition Tags"))
			.OnTextChanged(this, &SDaySequenceConditionSetPicker::OnSearchStringChanged)
		]

		// List of tags
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(VisibleConditionTagsListView, SListView<UClass*>)
			.ListItemsSource(&VisibleConditionTags)
			.OnGenerateRow(this, &SDaySequenceConditionSetPicker::OnGenerateRow)
			.SelectionMode(ESelectionMode::None)
			.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		]
	];


	
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/false, nullptr);
	
	MenuBuilder.BeginSection(FName(), LOCTEXT("SectionConditionSet", "Condition Set"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DaySequenceConditionSetPicker_ClearAllTags", "Clear All Tags"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
		FUIAction(FExecuteAction::CreateRaw(this, &SDaySequenceConditionSetPicker::OnUncheckAllTags))
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);
	
	MenuBuilder.EndSection();


	
	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> SDaySequenceConditionSetPicker::OnGenerateRow(UClass* InTag, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (const TObjectPtr<UDaySequenceConditionTag> SubclassCDO = Cast<UDaySequenceConditionTag>(InTag->GetDefaultObject()))
	{
		return SNew(STableRow<UClass*>, OwnerTable)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda([this, InTag](ECheckBoxState NewState)
			{
				if (NewState == ECheckBoxState::Checked)
				{
					OnTagChecked(InTag);
				}
				else if (NewState == ECheckBoxState::Unchecked)
				{
					OnTagUnchecked(InTag);
				}
			})
			.IsChecked_Lambda([this, InTag]()
			{
				return IsTagChecked(InTag);
			})
			.ToolTipText(FText::FromString(InTag->GetClassPathName().ToString()))
			.Content()
			[
				SNew(STextBlock)
				.Text(FText::FromString(SubclassCDO->GetConditionName()))
			]
		];
	}
	
	return SNew(STableRow<UClass*>, OwnerTable);
}

void SDaySequenceConditionSetPicker::OnSearchStringChanged(const FText& NewString)
{
	SearchString = NewString.ToString();
	RefreshListView();
}
	
void SDaySequenceConditionSetPicker::RefreshListView()
{
	VisibleConditionTags.Empty();

	// Add all tags matching the search string.
	for (UClass* Subclass : AllConditionTags)
	{
		if (const TObjectPtr<UDaySequenceConditionTag> SubclassCDO = Cast<UDaySequenceConditionTag>(Subclass->GetDefaultObject()))
		{
			if (SearchString.IsEmpty() || SubclassCDO->GetConditionName().Contains(SearchString))
			{
				VisibleConditionTags.AddUnique(Subclass);
			}
		}
	}

	// Lexicographically sort condition tags.
	Algo::Sort(VisibleConditionTags, [](const UClass* LHS, const UClass* RHS)
	{
		const TObjectPtr<UDaySequenceConditionTag> LHSCDO = Cast<UDaySequenceConditionTag>(LHS->GetDefaultObject());
		const TObjectPtr<UDaySequenceConditionTag> RHSCDO = Cast<UDaySequenceConditionTag>(RHS->GetDefaultObject());
			
		check (IsValid(LHSCDO) && IsValid(RHSCDO));
			
		return LHSCDO->GetConditionName() < RHSCDO->GetConditionName();
	});
	
	// Refresh the slate list.
	if (VisibleConditionTagsListView.IsValid())
	{
		VisibleConditionTagsListView->SetItemsSource(&VisibleConditionTags);
		VisibleConditionTagsListView->RequestListRefresh();
	}
}

void SDaySequenceConditionSetPicker::PopulateVisibleClasses()
{
	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	const FTopLevelAssetPath BaseAssetPath = UDaySequenceConditionTag::StaticClass()->GetClassPathName();
	TArray<FTopLevelAssetPath> NativeAssetPaths;
	NativeAssetPaths.AddUnique(BaseAssetPath);

	AllConditionTags.Reset();

	// Scan for native classes
	{
		const UClass* ParentClass = UDaySequenceConditionTag::StaticClass();

		auto IsNativeSubclass = [ParentClass](const UClass* InClass)
		{
			return InClass->IsChildOf(ParentClass)
				&& InClass != ParentClass
				&& (InClass->ClassFlags & CLASS_Native) != CLASS_None;
		};
		
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (UClass* CurrentClass = *It; CurrentClass && IsNativeSubclass(CurrentClass))
			{
				AllConditionTags.Add(CurrentClass);
				const FTopLevelAssetPath CurrentClassPath(CurrentClass->GetPathName());
				NativeAssetPaths.Add(CurrentClassPath);
			}
		}
	}
	
	// Scan for BP assets to get generated classes
    {
    	FARFilter Filter;
    	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
    	Filter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());
    	Filter.bRecursiveClasses = true;
    	Filter.TagsAndValues.Add(FBlueprintTags::NativeParentClassPath);
    
    	AssetRegistry.EnumerateAssets(Filter, [this, NativeAssetPaths](const FAssetData& Data)
    	{
    		for (auto CurrentAssetPath : NativeAssetPaths)
    		{
    			if (Data.TagsAndValues.FindTag(FBlueprintTags::NativeParentClassPath).AsExportPath().ToTopLevelAssetPath() == CurrentAssetPath)
    			{
					if (const UBlueprint* AssetAsBlueprint = Cast<UBlueprint>(Data.GetAsset()))
					{
						if (const TSubclassOf<UDaySequenceConditionTag> GeneratedClassAsConditionClass = *AssetAsBlueprint->GeneratedClass)
						{
							AllConditionTags.AddUnique(GeneratedClassAsConditionClass);
						}
					}
					else if (UBlueprintGeneratedClass* AssetAsBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Data.GetAsset()))
					{
						if (AssetAsBlueprintGeneratedClass->IsChildOf(UDaySequenceConditionTag::StaticClass()))
						{
							AllConditionTags.Add(AssetAsBlueprintGeneratedClass);
						}
					}
				}
    		}
    		return true;	// Returning false will halt the enumeration
    	});
    }
}

void SDaySequenceConditionSetPicker::PopulateCheckedTags()
{
	// Reset checked tag map
	for (UClass* Condition : AllConditionTags)
	{
		CheckedTags.FindOrAdd(Condition) = false;
	}

	// Access condition set array
	void* StructPointer = nullptr;
	if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
	{
		FDaySequenceConditionSet& ConditionSet = *static_cast<FDaySequenceConditionSet*>(StructPointer);
		FDaySequenceConditionSet::FConditionValueMap& Conditions = ConditionSet.Conditions;

		HelperConditionSet->SetConditions(Conditions);
		
		// Bring checked tag map to parity with current condition set
		for (TTuple<TSubclassOf<UDaySequenceConditionTag>, bool>& Element : Conditions)
		{
			TSubclassOf<UDaySequenceConditionTag>& Subclass = Element.Key;
			CheckedTags[Subclass] = true;
		}
	}
}

ECheckBoxState SDaySequenceConditionSetPicker::IsTagChecked(UClass* InTag)
{
	// This will add InTag to the map if not in it with a default value of false.
	return CheckedTags.FindOrAdd(InTag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDaySequenceConditionSetPicker::OnTagChecked(UClass* InTag)
{
	CheckedTags.FindOrAdd(InTag) = true;
	HelperConditionSet->GetConditions().FindOrAdd(InTag) = true;	// Expected value defaults to true when adding a condition.
	FlushHelperConditionSet();
}

void SDaySequenceConditionSetPicker::OnTagUnchecked(UClass* InTag)
{
	CheckedTags.FindOrAdd(InTag) = false;
	HelperConditionSet->GetConditions().Remove(InTag);
	FlushHelperConditionSet();
}

void SDaySequenceConditionSetPicker::OnUncheckAllTags()
{
	Algo::ForEach(AllConditionTags, [this](UClass* Condition) { CheckedTags.FindOrAdd(Condition) = false; });
	HelperConditionSet->GetConditions().Empty();
	FlushHelperConditionSet();
}

void SDaySequenceConditionSetPicker::FlushHelperConditionSet() const
{
	// Set the property with a formatted string in order to propagate CDO changes to instances if necessary
	const FString OutString = HelperConditionSet->GetConditionSetExportText();
	StructPropertyHandle->SetValueFromFormattedString(OutString);
}

#undef LOCTEXT_NAMESPACE
