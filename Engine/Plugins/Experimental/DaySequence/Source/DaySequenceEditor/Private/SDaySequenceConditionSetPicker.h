// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
enum class ECheckBoxState : uint8;
template <typename ItemType> class SListView;

class UEditableDaySequenceConditionSet;
class IPropertyHandle;
class SComboButton;

/**
 * Widget allowing user to edit the condition tags in a given condition set.
 */
class SDaySequenceConditionSetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDaySequenceConditionSetPicker)
		: _StructPropertyHandle(nullptr)
	{}
		// Used for writing changes to the condition set being edited. 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, StructPropertyHandle)
	SLATE_END_ARGS()
	
	/** Construct the actual widget */
	DAYSEQUENCEEDITOR_API void Construct(const FArguments& InArgs);

private:
	TSharedRef<SWidget> GetChildWidget();
	TSharedRef<ITableRow> OnGenerateRow(UClass* InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Members for tracking and using search text. */
	void OnSearchStringChanged(const FText& NewString);
	TSharedPtr<SSearchBox> ConditionSearchBox;
	FString SearchString;
	
	/** Populate VisibleConditionTags with the correct subset of AllConditionTags given our current search text. */
	void RefreshListView();

	/** The set of condition tags to display, based on our current search text. */
	TArray<UClass*> VisibleConditionTags;
	TSharedPtr<SListView<UClass*>> VisibleConditionTagsListView;

	/** Queries the asset registry to get all known condition tags. */
	void PopulateVisibleClasses();

	/** The set of condition tags which can be displayed. */
	TArray<UClass*> AllConditionTags;

	/** Functions for querying the condition set for information about a particular tag. */
	ECheckBoxState IsTagChecked(UClass* InTag);
	void PopulateCheckedTags();
	TMap<UClass*, bool> CheckedTags;

	/** Functions for adding/removing tags to/from the condition set. */
	void OnTagChecked(UClass* NodeChecked);
	void OnTagUnchecked(UClass* NodeUnchecked);
	void OnUncheckAllTags();

	/** Brings the source condition set to parity with HelperConditionSet. Generally called immediately after modifying HelperConditionSet. */
	void FlushHelperConditionSet() const;
	
	/** Property handle to an FDaySequenceConditionSet, used for accessing the source condition set. */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** A helper class which is used for propagating changes to the source condition set. */
	TStrongObjectPtr<UEditableDaySequenceConditionSet> HelperConditionSet;
};
