// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDaySequenceConditionTagChip.h"

#include "Styling/StyleColors.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "DaySequenceConditionTagChip"

//------------------------------------------------------------------------------
// SDaySequenceConditionTagChip
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SDaySequenceConditionTagChip)
void SDaySequenceConditionTagChip::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Text", TextAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ToolTipText", ToolTipTextAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ExpectedValue", ExpectedValueAttribute, EInvalidateWidgetReason::Layout);
}

SDaySequenceConditionTagChip::SDaySequenceConditionTagChip()
	: ToolTipTextAttribute(*this)
	, TextAttribute(*this)
	, ExpectedValueAttribute(*this)
{}

void SDaySequenceConditionTagChip::Construct(const FArguments& InArgs)
{
	ToolTipTextAttribute.Assign(*this, InArgs._ToolTipText);
	TextAttribute.Assign(*this, InArgs._Text);
	ExpectedValueAttribute.Assign(*this, InArgs._ExpectedValue);
	OnClearPressed = InArgs._OnClearPressed;
	OnExpectedValueChanged = InArgs._OnExpectedValueChanged;
	TagClass = InArgs._TagClass;

	TWeakPtr<SDaySequenceConditionTagChip> WeakSelf = StaticCastWeakPtr<SDaySequenceConditionTagChip>(AsWeak());
	
	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(ChipHeight)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.OnClicked_Lambda([WeakSelf]()
			{
				return FReply::Unhandled();
			})
			[
				SNew(SHorizontalBox)

				// Expected Value Checkbox
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(0.f)
				[
					SNew(SCheckBox)
					.ToolTipText_Lambda([this]()
					{
						return FText::FromString(FString::Printf(TEXT("Determines what value the condition must return for it to be considered 'passing' for this sequence. Current Expected Value: %hs"), ExpectedValueAttribute.Get() ? "True" : "False"));
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						OnExpectedValueChanged.Execute(TagClass, NewState == ECheckBoxState::Checked);
					})
					.IsChecked_Lambda([this]()
					{
						return ExpectedValueAttribute.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				]
				
				// Condition Name
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(10.f, 0.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
					.Text_Lambda([WeakSelf]()
					{
						const TSharedPtr<SDaySequenceConditionTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? Self->TextAttribute.Get() : FText::GetEmpty();
					})
					.ToolTipText_Lambda([WeakSelf]()
					{
						const TSharedPtr<SDaySequenceConditionTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? Self->ToolTipTextAttribute.Get() : FText::GetEmpty();
					})
				]

				// Clear Button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.f)
				[
					SAssignNew(ClearButton, SButton)
					.Visibility_Lambda([WeakSelf]()
					{
						const TSharedPtr<SDaySequenceConditionTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
					})
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.ToolTipText(LOCTEXT("ClearTag", "Clear Tag"))
					.ContentPadding(0)
					.OnClicked_Lambda([WeakSelf]()
					{
						const TSharedPtr<SDaySequenceConditionTagChip> Self = WeakSelf.Pin();
						if (Self.IsValid() && Self->OnClearPressed.IsBound())
						{
							return Self->OnClearPressed.Execute();
						}
						return FReply::Unhandled();
					})
					[
						SNew(SImage)
						.ColorAndOpacity_Lambda([WeakSelf]()
						{
							const TSharedPtr<SDaySequenceConditionTagChip> Self = WeakSelf.Pin();
							if (Self.IsValid() && Self->ClearButton.IsValid())
							{
								return Self->ClearButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
							}
							return FStyleColors::Foreground;
						})
						.Image(FAppStyle::GetBrush("Icons.X"))
						.DesiredSizeOverride(FVector2D(12.f, 12.f))
					]
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
