// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class SButton;

/**
 * Widget for displaying a single condition tag.
 */
class SDaySequenceConditionTagChip : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDaySequenceConditionTagChip, SCompoundWidget)

public:

	DECLARE_DELEGATE_RetVal(FReply, FOnClearPressed);
	DECLARE_DELEGATE_TwoParams(FOnExpectedValueChanged, UClass*, bool)

	SLATE_BEGIN_ARGS(SDaySequenceConditionTagChip)
	{}
		// The condition subclass associated with this chip.
		SLATE_ARGUMENT(UClass*, TagClass)
	
		// Callback for when clear tag button is pressed.
		SLATE_EVENT(FOnClearPressed, OnClearPressed)

		// Callback for when ExpectedValue has changed.
		SLATE_EVENT(FOnExpectedValueChanged, OnExpectedValueChanged)

		// The value which is expected for this condition to be true.
		SLATE_ATTRIBUTE(bool, ExpectedValue)
	
		// Tooltip to display.
		SLATE_ATTRIBUTE(FText, ToolTipText)

		// Text to display.
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS();

	DAYSEQUENCEEDITOR_API SDaySequenceConditionTagChip();
	DAYSEQUENCEEDITOR_API void Construct(const FArguments& InArgs);

	/** This is public because it is used in SDaySequenceConditionSetCombo and SDaySequenceConditionSetPicker. */
	inline static constexpr float ChipHeight = 25.f;
	
private: 
	TSlateAttribute<FText> ToolTipTextAttribute;
	TSlateAttribute<FText> TextAttribute;
	TSlateAttribute<bool> ExpectedValueAttribute;
	
	FOnClearPressed OnClearPressed;
	FOnExpectedValueChanged OnExpectedValueChanged;
	
	TSharedPtr<SButton> ClearButton;
	
	UClass* TagClass;
};
