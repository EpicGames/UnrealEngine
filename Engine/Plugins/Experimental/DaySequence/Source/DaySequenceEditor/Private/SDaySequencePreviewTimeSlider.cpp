// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDaySequencePreviewTimeSlider.h"

#include "DaySequenceActor.h"
#include "DaySequenceActorPreview.h"
#include "IDaySequenceEditorModule.h"

#include "Modules/ModuleManager.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/SBoxPanel.h"

void SDaySequencePreviewTimeSlider::Construct(const FArguments& InArgs)
{
	IDaySequenceEditorModule* DaySequenceEditorModule = FModuleManager::GetModulePtr<IDaySequenceEditorModule>("DaySequenceEditor");
	DaySequenceActorPreview = &DaySequenceEditorModule->GetDaySequenceActorPreview();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot().AutoHeight()
		.Padding(0, 5, 10, 5)
		[
			SNew(SSimpleTimeSlider)
			.IsEnabled_Lambda([this]()
			{
				return DaySequenceActorPreview->IsPreviewEnabled();
			})
			.AllowPan(false)
			.AllowZoom(false)
			.DesiredSize({100,24})
			.ViewRange_Lambda([this]()
			{
				return TRange<double>(0.,DaySequenceActorPreview->GetDayLength());
			})
			.ClampRange_Lambda([this]()
			{
				return TRange<double>(0.,DaySequenceActorPreview->GetDayLength());
			})
			.ClampRangeHighlightSize(0.15f)
			.ClampRangeHighlightColor_Lambda([this]()
			{
				return (DaySequenceActorPreview->IsPreviewEnabled() ? FLinearColor::Red.CopyWithNewOpacity(0.5f) : FLinearColor::Gray.CopyWithNewOpacity(0.5f));
			})
			.ScrubPosition_Lambda([this]()
			{
				return DaySequenceActorPreview->GetPreviewTime();
			})
			// TODO: Hook into FLevelEditorSequencerIntegration to defer details updates (OnBeginDeferUpdates/OnEndDeferUpdates)
			//.OnBeginScrubberMovement_Lambda([this](){})
			//.OnEndScrubberMovement_Lambda([this](){})
			.OnScrubPositionChanged_Lambda(
				[this](double NewScrubTime, bool /*bIsScrubbing*/)
				{
					if (ADaySequenceActor* PreviewActor = DaySequenceActorPreview->GetPreviewActor().Get())
					{
						PreviewActor->SetTimeOfDayPreview(NewScrubTime);
					}
				})
		]
		+SVerticalBox::Slot().AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(0, 5, 10, 5)
		[
			DaySequenceActorPreview->MakeTransportControls(false)
		]
	];
}

