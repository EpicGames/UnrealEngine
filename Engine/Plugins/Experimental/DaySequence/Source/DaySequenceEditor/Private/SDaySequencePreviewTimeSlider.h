// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDaySequenceActorPreview;

class SDaySequencePreviewTimeSlider : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDaySequencePreviewTimeSlider) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FDaySequenceActorPreview* DaySequenceActorPreview = nullptr;
};

