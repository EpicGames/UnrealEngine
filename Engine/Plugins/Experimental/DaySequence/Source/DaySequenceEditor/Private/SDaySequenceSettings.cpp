// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDaySequenceSettings.h"
#include "DetailsViewArgs.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Editor.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "IDetailsView.h"
#include "SSubobjectInstanceEditor.h"
#include "DaySequenceActor.h"
#include "DaySequenceSubsystem.h"
#include "ISCSEditorUICustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "DaySequenceEditor"

class FDaySequenceSCSEditorUICustomization : public ISCSEditorUICustomization
{
public:
	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const override { return true; }
	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override { return true; }
};

enum class ESettingsSection : uint8
{
	Environment,
	TimeOfDay,
};

void SDaySequenceSettings::Construct(const FArguments& InArgs)
{
	FEditorDelegates::MapChange.AddSP(this, &SDaySequenceSettings::OnMapChanged);

	UpdateDaySequenceActor();

	TSharedRef<SSubobjectInstanceEditor> SubObjectEditorRef
		= SNew(SSubobjectInstanceEditor)
		.ObjectContext(this, &SDaySequenceSettings::GetObjectContext)
		//.AllowEditing(this, &SActorDetails::GetAllowComponentTreeEditing)
		.OnSelectionUpdated(this, &SDaySequenceSettings::OnSubobjectEditorTreeViewSelectionChanged);
		//.OnItemDoubleClicked(this, &SActorDetails::OnSubobjectEditorTreeViewItemDoubleClicked)
		//.OnObjectReplaced(this, &SActorDetails::OnSubobjectEditorTreeViewObjectReplaced);
	
	SubObjectEditorRef->SetUICustomization(MakeShared<FDaySequenceSCSEditorUICustomization>());

	SubobjectEditor = SubObjectEditorRef;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	ComponentDetailsView = DetailsView;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(10, 4)
		.AutoHeight()
		[
			SNew(SSegmentedControl<ESettingsSection>)
			.OnValueChanged(this, &SDaySequenceSettings::OnSettingsSectionChanged)
			+ SSegmentedControl<ESettingsSection>::Slot(ESettingsSection::Environment)
			.Text(LOCTEXT("EnvironmentSettings", "Environment"))
			.ToolTip(LOCTEXT("EnvironmentSettings_ToolTip", "Set up the Day environment"))
			+ SSegmentedControl<ESettingsSection>::Slot(ESettingsSection::TimeOfDay)
			.Text(LOCTEXT("DaySequenceSettings", "Time of Day"))
			.ToolTip(LOCTEXT("DaySequenceSettings_ToolTip", "Specify time of day settings"))
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew(SettingsSwitcher, SWidgetSwitcher)
			+ SWidgetSwitcher::Slot()
			[
				MakeEnvironmentPanel()
			]
			+ SWidgetSwitcher::Slot()
			[
				MakeEditDaySequencePanel()
			]
		]
	];

	SettingsSwitcher->SetActiveWidgetIndex(0);
}

void SDaySequenceSettings::OnSettingsSectionChanged(ESettingsSection NewSection)
{
	SettingsSwitcher->SetActiveWidgetIndex(NewSection == ESettingsSection::Environment ? 0 : 1);
}

void SDaySequenceSettings::OnMapChanged(uint32 Flags)
{
	if (Flags == MapChangeEventFlags::NewMap)
	{
		UpdateDaySequenceActor();
	}
}

UObject* SDaySequenceSettings::GetObjectContext() const
{
	return EditorDaySequenceActor.Get();
}

void SDaySequenceSettings::UpdateDaySequenceActor()
{
	EditorDaySequenceActor = nullptr;

	if (GEditor)
	{
		UDaySequenceSubsystem* TODSubsystem = GEditor->GetEditorWorldContext().World()->GetSubsystem<UDaySequenceSubsystem>();
		EditorDaySequenceActor = TODSubsystem->GetDaySequenceActor();
	}
}

FReply SDaySequenceSettings::OnEditDaySequenceClicked()
{
	return FReply::Handled();
}

void SDaySequenceSettings::OnSubobjectEditorTreeViewSelectionChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode> >& SelectedNodes)
{
	TArray<UObject*> Objects;
	Objects.Reserve(SelectedNodes.Num());

	for (const TSharedPtr<FSubobjectEditorTreeNode>& Node : SelectedNodes)
	{
		if (const UObject* Object = Node->GetObject())
		{
			Objects.Add(const_cast<UObject*>(Object));
		}
	}

	ComponentDetailsView->SetObjects(Objects);
}

TSharedRef<SWidget> SDaySequenceSettings::MakeEnvironmentPanel()
{
	check(SubobjectEditor.IsValid() && ComponentDetailsView.IsValid());
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10, 4, 4, 4)
			[
				SubobjectEditor->GetToolButtonsBox().ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("EditDaySequence", "Edit Day Sequence"))
				.OnClicked(this, &SDaySequenceSettings::OnEditDaySequenceClicked)
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+ SSplitter::Slot()
			[
				SubobjectEditor.ToSharedRef()
			]
			+ SSplitter::Slot()
			[
				ComponentDetailsView.ToSharedRef()
			]		
		];
}

TSharedRef<SWidget> SDaySequenceSettings::MakeEditDaySequencePanel()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight();
}


#undef LOCTEXT_NAMESPACE

