// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

enum class ESettingsSection : uint8;

class SWidgetSwitcher;
class SSubobjectInstanceEditor;
class ADaySequenceActor;
class IDetailsView;
class FSubobjectEditorTreeNode;

class SDaySequenceSettings : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDaySequenceSettings)
	{}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

private:
	void OnSettingsSectionChanged(ESettingsSection NewSection);
	void OnMapChanged(uint32 Flags);
	UObject* GetObjectContext() const;
	void UpdateDaySequenceActor();
	FReply OnEditDaySequenceClicked();
	void OnSubobjectEditorTreeViewSelectionChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode> >& SelectedNodes);

	TSharedRef<SWidget> MakeEnvironmentPanel();
	TSharedRef<SWidget> MakeEditDaySequencePanel();
private:
	TSharedPtr<SWidgetSwitcher> SettingsSwitcher;
	TSharedPtr<SSubobjectInstanceEditor> SubobjectEditor;
	TWeakObjectPtr<ADaySequenceActor> EditorDaySequenceActor;
	TSharedPtr<IDetailsView> ComponentDetailsView;
};