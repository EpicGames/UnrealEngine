// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/ValueOrError.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSpawnRegister.h"
#include "DaySequenceSpawnRegister.h"
#include "UObject/ObjectKey.h"

#define UE_API DAYSEQUENCEEDITOR_API

struct FNewSpawnable;

class IMovieScenePlayer;
class ISequencer;
class UMovieScene;
class UMovieSceneSequence;
class FObjectPreSaveContext;

/**
 * Spawn register used in the editor to add some usability features like maintaining selection states, and projecting spawned state onto spawnable defaults
 */
class FDaySequenceEditorSpawnRegister
	: public FDaySequenceSpawnRegister
{
public:

	/** Constructor */
	UE_API FDaySequenceEditorSpawnRegister();

	/** Destructor. */
	UE_API ~FDaySequenceEditorSpawnRegister() override;

public:
	UE_API void SetSequencer(const TSharedPtr<ISequencer>& Sequencer);

public:
	// FDaySequenceSpawnRegister interface
	UE_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID) override;
#if WITH_EDITOR
	UE_API virtual TValueOrError<FNewSpawnable, FText> CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory = nullptr) override;
	UE_API virtual void SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) override;
	UE_API virtual void HandleConvertPossessableToSpawnable(UObject* OldObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TOptional<FTransformData>& OutTransformData) override;
	UE_API virtual bool CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const override;
#endif

private:
	/** Called when the editor selection has changed. */
	UE_API void HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

#if WITH_EDITOR
	/** Saves the default state for the specified spawnable, if an instance for it currently exists */
	UE_API void SaveDefaultSpawnableState(const FGuid& Guid, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	UE_API void SaveDefaultSpawnableStateImpl(FMovieSceneSpawnable& Spawnable, UMovieSceneSequence* Sequence, UObject* SpawnedObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);
#endif

	/** Called from the editor when a blueprint object replacement has occurred */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Called whenever an object is modified in the editor */
	UE_API void OnObjectModified(UObject* ModifiedObject);

	/** Called before an object is saved in the editor */
	UE_API void OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext);

private:
	struct FTrackedObjectState
	{
		FTrackedObjectState(FMovieSceneSequenceIDRef InTemplateID, const FGuid& InObjectBindingID) : TemplateID(InTemplateID), ObjectBindingID(InObjectBindingID), bHasBeenModified(false) {}

		/** The sequence ID that spawned this object */
		FMovieSceneSequenceID TemplateID;

		/** The object binding ID of the object in the template */
		FGuid ObjectBindingID;

		/** true if this object has been modified since it was spawned and is different from the current object template */
		bool bHasBeenModified;
	};

private:
	/** Handles for delegates that we've bound to. */
	FDelegateHandle OnActorSelectionChangedHandle;

	/** Set of spawn register keys for objects that should be selected if they are spawned. */
	TSet<FMovieSceneSpawnRegisterKey> SelectedSpawnedObjects;

	/** Map from a sequenceID to an array of objects that have been tracked */
	TMap<FObjectKey, FTrackedObjectState> TrackedObjects;

	/** Set of UMovieSceneSequences that this register has spawned objects for that are modified */
	TSet<FObjectKey> SequencesWithModifiedObjects;

	/** True if we should clear the above selection cache when the editor selection has been changed. */
	bool bShouldClearSelectionCache;

	/** Weak pointer to the active sequencer. */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Handle to a delegate that is bound to FCoreUObjectDelegates::OnObjectModified to harvest changes to spawned objects. */
	FDelegateHandle OnObjectModifiedHandle;

	/** Handle to a delegate that is bound to FCoreUObjectDelegates::OnObjectPreSave to harvest changes to spawned objects. */
	FDelegateHandle OnObjectSavedHandle;
};

#undef UE_API
