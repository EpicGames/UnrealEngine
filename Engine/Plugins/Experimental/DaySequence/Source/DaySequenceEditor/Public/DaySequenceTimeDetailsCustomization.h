// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

/**
 *  Customize FDaySequenceTime
 */
class FDaySequenceTimeDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDaySequenceTimeDetailsCustomization>();
	}

	FDaySequenceTimeDetailsCustomization()
	{}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	FText OnGetTimeText() const;
	void OnTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Store the property handle to the Time field so we can get/set the value on the object via text box callbacks. */
	TSharedPtr<IPropertyHandle> TimeProperty;
};
