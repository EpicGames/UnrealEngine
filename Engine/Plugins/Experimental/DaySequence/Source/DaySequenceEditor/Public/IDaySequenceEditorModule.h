// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class FDaySequenceActorPreview;

/** Interface for the DaySequence Editor module. */
class IDaySequenceEditorModule : public IModuleInterface
{
public:
	// TODO: Should we set this for UMoviePipelineQueueSubsystem like LevelSequence?
	DECLARE_MULTICAST_DELEGATE_OneParam(FAllowPlaybackContext, bool&);
	virtual FAllowPlaybackContext& OnComputePlaybackContext() = 0;

	virtual FDaySequenceActorPreview& GetDaySequenceActorPreview() = 0;

	DECLARE_MULTICAST_DELEGATE(FPreSelectDaySequenceActor);
	virtual FPreSelectDaySequenceActor& OnPreSelectDaySequenceActor() = 0;

	DECLARE_MULTICAST_DELEGATE(FPostSelectDaySequenceActor);
	virtual FPostSelectDaySequenceActor& OnPostSelectDaySequenceActor() = 0;
};

